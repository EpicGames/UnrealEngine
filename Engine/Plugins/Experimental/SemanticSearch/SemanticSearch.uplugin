{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Semantic Search",
	"Description": "Very early work in progress of a semantic search system for assets.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "https://dev.epicgames.com/community/",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EditorOnly": true,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "SemanticSearch",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"TargetAllowList": ["Editor"]
		},
		{
			"Name": "SemanticSearchEditorIntegrations",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"TargetAllowList": ["Editor"]
		}
	]
}