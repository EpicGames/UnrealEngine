// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetProcessorManager.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ISemanticSearchModule.h"
#include "Misc/ScopeRWLock.h"
#include "SemanticSearchDerivedDataBuilder.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"

#include <atomic>

namespace UE::SemanticSearch
{
	using namespace UE::DerivedData;
	FAssetProcessorManager& FAssetProcessorManager::Get()
	{
		static FAssetProcessorManager ManagerSingleton;
		return ManagerSingleton;
	}

	FAssetProcessorManager::FAssetProcessorManager()
	{
		OnObjectsReinstancedHandle = FCoreUObjectDelegates::OnObjectsReinstanced.AddRaw(this, &FAssetProcessorManager::OnObjectsReinstanced);
		OnModuleUnloadedHandle = FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.AddRaw(this, &FAssetProcessorManager::OnModuleUnloaded);
		OnHotReloadCompleteHandle = FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.AddRaw(this, &FAssetProcessorManager::OnHotReloadComplete);

		if (FAssetRegistryModule* ARM = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
		{
			IAssetRegistry& AR = ARM->Get();
			OnAssetAddedHandle = AR.OnAssetAdded().AddRaw(this, &FAssetProcessorManager::InvalidateSupportedAssetCount);
			OnAssetRemovedHandle = AR.OnAssetRemoved().AddRaw(this, &FAssetProcessorManager::InvalidateSupportedAssetCount);
		}
	}
	
	FAssetProcessorManager::~FAssetProcessorManager()
	{
		FCoreUObjectDelegates::OnObjectsReinstanced.Remove(OnObjectsReinstancedHandle);
		FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.Remove(OnModuleUnloadedHandle);
		FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.Remove(OnHotReloadCompleteHandle);

		if (FAssetRegistryModule* ARM = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
		{
			IAssetRegistry& AR = ARM->Get();
			AR.OnAssetAdded().Remove(OnAssetAddedHandle);
			AR.OnAssetRemoved().Remove(OnAssetRemovedHandle);
		}
	}

	TSharedPtr<IAssetProcessor> FAssetProcessorManager::GetProcessorForAsset(const FAssetData& InAssetData) const
	{
		FReadScopeLock ReadLock(AssetTypeToProcessorLock);
		if (const TSharedPtr<IAssetProcessor>* Ptr = AssetTypeToProcessor.Find(InAssetData.AssetClassPath))
		{
			return *Ptr;
		}
		return nullptr;
	}

	TSet<FName> FAssetProcessorManager::GetSupportedClassNames() const
	{
		FReadScopeLock ReadLock(AssetTypeToProcessorLock);
		TSet<FName> Out;
		Out.Reserve(AssetTypeToProcessor.Num());
		for (const TPair<FTopLevelAssetPath, TSharedPtr<IAssetProcessor>>& Pair : AssetTypeToProcessor)
		{
			if (Pair.Value)
			{
				Out.Add(Pair.Key.GetAssetName());
			}
		}
		return Out;
	}

	int32 FAssetProcessorManager::GetSupportedAssetCount() const
	{
		if (CachedSupportedAssetCount < 0)
		{
			CachedSupportedAssetCount = ComputeSupportedAssetCount();
		}
		return CachedSupportedAssetCount;
	}

	void FAssetProcessorManager::InvalidateSupportedAssetCount(const FAssetData&)
	{
		CachedSupportedAssetCount = -1;
	}

	void FAssetProcessorManager::InvalidateSupportedAssetCount()
	{
		CachedSupportedAssetCount = -1;
	}

	void FAssetProcessorManager::ClearBuildCache()
	{
		BuildsJobs.Empty();
	}

	int32 FAssetProcessorManager::ComputeSupportedAssetCount() const
	{
		// Collect all unique class paths that have a processor entry.
		// Derived classes are pre-populated by PopulateChildClassesCache.
		TSet<FTopLevelAssetPath> SupportedClassPaths;
		{
			FReadScopeLock ReadLock(AssetTypeToProcessorLock);
			for (const auto& Pair : AssetTypeToProcessor)
			{
				if (Pair.Value)
				{
					SupportedClassPaths.Add(Pair.Key);
				}
			}
		}

		int32 Total = 0;
		IAssetRegistry& AR = IAssetRegistry::GetChecked();
		for (const FTopLevelAssetPath& ClassPath : SupportedClassPaths)
		{
			TArray<FAssetData> Assets;
			AR.GetAssetsByClass(ClassPath, Assets, /*bSearchSubClasses=*/false);
			for (const FAssetData& Asset : Assets)
			{
				if (!Asset.IsRedirector() && ISemanticSearchModule::IsInIndexedFolder(Asset))
				{
					++Total;
				}
			}
		}
		return Total;
	}

	void FAssetProcessorManager::CancelAllTasks()
	{
		BuildsJobs.ForEach([](auto& Pair) { Pair.Value->KeepAlive(); });
		BuildsJobs.ForEach([](auto& Pair) { Pair.Value->Cancel(); });
		BuildsJobs.ForEach([](auto& Pair) { Pair.Value->Wait(); });
		BuildsJobs.Empty();
	}

	bool FAssetProcessorManager::GetOrCreateTask(const FAssetData& InAssetData, TFunctionRef<void(const TSharedRef<Private::FSemanticSearchBuildCacheTask>&)> InApply)
	{
		TSharedPtr<IAssetProcessor> AssetProcessor = GetProcessorForAsset(InAssetData);
		if (!AssetProcessor)
		{
			return false;
		}

		FCacheKey CacheKey = Private::FSemanticSearchBuildCacheTask::GenerateCacheKey(InAssetData, AssetProcessor.ToSharedRef());
		if (CacheKey == FCacheKey::Empty)
		{
			return false;
		}

		BuildsJobs.FindOrProduceAndApply(
			CacheKey,
			[&InAssetData, &AssetProcessor, &CacheKey]() -> TSharedRef<Private::FSemanticSearchBuildCacheTask>
			{
				return Private::FSemanticSearchBuildCacheTask::Create(InAssetData, AssetProcessor.ToSharedRef(), MoveTemp(CacheKey)).ToSharedRef();
			},
			InApply);

		return true;
	}

	bool FAssetProcessorManager::GetCaptionData(const FAssetData& InAssetData, TUniqueFunction<void(FAssetCaptionResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable)
	{
		return GetOrCreateTask(InAssetData, [&InOnDataAvailable](const TSharedRef<Private::FSemanticSearchBuildCacheTask>& Task)
		{
			Task->GetCaptionData([OnDataAvailable = MoveTemp(InOnDataAvailable)](Private::FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
			{
				FAssetCaptionResult Result;
				Result.Caption   = MoveTemp(CaptionData.Caption);
				Result.Keywords  = MoveTemp(CaptionData.Keywords);
				OnDataAvailable(MoveTemp(Result), MoveTemp(ErrorMessage), Reason);
			});
		});
	}

	bool FAssetProcessorManager::GetQuantizedEmbeddingData(const FAssetData& InAssetData, const FIoHash& CodebookHash,
		TUniqueFunction<void(FAssetQuantizedEmbeddingResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable, bool bBuildOnMiss)
	{
		return GetOrCreateTask(InAssetData, [&InOnDataAvailable, &CodebookHash, bBuildOnMiss](const TSharedRef<Private::FSemanticSearchBuildCacheTask>& Task)
		{
			Task->GetQuantizedEmbeddingData(CodebookHash,
				[OnDataAvailable = MoveTemp(InOnDataAvailable)](Private::FQuantizedEmbeddingDerivedData&& QuantizedData, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
				{
					FAssetQuantizedEmbeddingResult Result;
					Result.QuantizedCodes = MoveTemp(QuantizedData.QuantizedEmbedding);
					OnDataAvailable(MoveTemp(Result), MoveTemp(ErrorMessage), Reason);
				}, bBuildOnMiss);
		});
	}

	bool FAssetProcessorManager::GetEmbeddingData(const FAssetData& InAssetData, TUniqueFunction<void(FAssetEmbeddingResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable, bool bBuildOnMiss)
	{
		return GetOrCreateTask(InAssetData, [&InOnDataAvailable, bBuildOnMiss](const TSharedRef<Private::FSemanticSearchBuildCacheTask>& Task)
		{
			Task->GetEmbeddingData([OnDataAvailable = MoveTemp(InOnDataAvailable)](Private::FEmbeddingDerivedData&& EmbeddingData, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
			{
				FAssetEmbeddingResult Result;
				Result.Embedding = MoveTemp(EmbeddingData.Embedding);
				OnDataAvailable(MoveTemp(Result), MoveTemp(ErrorMessage), Reason);
			}, bBuildOnMiss);
		});
	}

	/**
	 * Shared completion tracker for combined indexing data fetches.
	 * Holds partial results from caption and embedding/quantized callbacks,
	 * fires the final callback when both have completed.
	 */
	struct FIndexingDataTracker
	{
		TUniqueFunction<void(FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)> Callback;
		FCriticalSection Lock;
		FAssetIndexingResult Result;
		FString Error;
		EAssetIndexFailureReason Reason = EAssetIndexFailureReason::None;
		std::atomic<int32> CompletedCount{0};

		explicit FIndexingDataTracker(TUniqueFunction<void(FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)>&& InCallback)
			: Callback(MoveTemp(InCallback)) {}

		void OnCaptionComplete(FAssetCaptionResult&& InCaption, FString&& InError, EAssetIndexFailureReason InReason)
		{
			{
				FScopeLock ScopeLock(&Lock);
				Result.Caption = MoveTemp(InCaption.Caption);
				Result.Keywords = MoveTemp(InCaption.Keywords);
				if (!InError.IsEmpty() && Error.IsEmpty()) { Error = MoveTemp(InError); Reason = InReason; }
			}
			TryFireCallback();
		}

		void OnEmbeddingComplete(FAssetEmbeddingResult&& InEmbedding, FString&& InError, EAssetIndexFailureReason InReason)
		{
			{
				FScopeLock ScopeLock(&Lock);
				Result.Embedding = MoveTemp(InEmbedding.Embedding);
				if (!InError.IsEmpty() && Error.IsEmpty()) { Error = MoveTemp(InError); Reason = InReason; }
			}
			TryFireCallback();
		}

		void OnQuantizedComplete(FAssetQuantizedEmbeddingResult&& InQuantized, FString&& InError, EAssetIndexFailureReason InReason)
		{
			{
				FScopeLock ScopeLock(&Lock);
				Result.QuantizedCodes = MoveTemp(InQuantized.QuantizedCodes);
				if (!InError.IsEmpty() && Error.IsEmpty()) { Error = MoveTemp(InError); Reason = InReason; }
			}
			TryFireCallback();
		}

	private:
		void TryFireCallback()
		{
			if (CompletedCount.fetch_add(1, std::memory_order_acq_rel) == 1)
			{
				Callback(MoveTemp(Result), MoveTemp(Error), Reason);
			}
		}
	};

	bool FAssetProcessorManager::GetIndexingData(const FAssetData& InAssetData,
		TUniqueFunction<void(FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable, bool bBuildOnMiss)
	{
		return GetOrCreateTask(InAssetData,
			[&InOnDataAvailable, bBuildOnMiss](const TSharedRef<Private::FSemanticSearchBuildCacheTask>& Task)
			{
				Task->GetRecordData(
					[OnDataAvailable = MoveTemp(InOnDataAvailable)](
						Private::FEmbeddingDerivedData&& EmbeddingData,
						Private::FCaptionDerivedData&& CaptionData,
						FString&& ErrorMessage,
						EAssetIndexFailureReason Reason) mutable
					{
						FAssetIndexingResult Result;
						Result.Caption = MoveTemp(CaptionData.Caption);
						Result.Keywords = MoveTemp(CaptionData.Keywords);
						Result.Embedding = MoveTemp(EmbeddingData.Embedding);
						OnDataAvailable(MoveTemp(Result), MoveTemp(ErrorMessage), Reason);
					}, bBuildOnMiss);
			});
	}

	bool FAssetProcessorManager::GetQuantizedIndexingData(const FAssetData& InAssetData, const FIoHash& CodebookHash,
		TUniqueFunction<void(FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable, bool bBuildOnMiss)
	{
		TSharedRef<FIndexingDataTracker> Tracker = MakeShared<FIndexingDataTracker>(MoveTemp(InOnDataAvailable));

		return GetOrCreateTask(InAssetData,
			[&Tracker, &CodebookHash, bBuildOnMiss](const TSharedRef<Private::FSemanticSearchBuildCacheTask>& Task)
			{
				Task->GetCaptionData(
					[Tracker](Private::FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
					{
						FAssetCaptionResult Result;
						Result.Caption = MoveTemp(CaptionData.Caption);
						Result.Keywords = MoveTemp(CaptionData.Keywords);
						Tracker->OnCaptionComplete(MoveTemp(Result), MoveTemp(ErrorMessage), Reason);
					});

				Task->GetQuantizedEmbeddingData(CodebookHash,
					[Tracker](Private::FQuantizedEmbeddingDerivedData&& QuantizedData, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
					{
						FAssetQuantizedEmbeddingResult Result;
						Result.QuantizedCodes = MoveTemp(QuantizedData.QuantizedEmbedding);
						Tracker->OnQuantizedComplete(MoveTemp(Result), MoveTemp(ErrorMessage), Reason);
					}, bBuildOnMiss);
			});
	}

	void FAssetProcessorManager::BatchGetIndexingData(
		TConstArrayView<FAssetData> Assets,
		TFunction<void(const FAssetData& Asset, FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)> OnAssetResult,
		bool bBuildOnMiss)
	{
		using namespace DerivedData;

		TArray<Private::FBatchDDCEntry> Entries;
		Entries.Reserve(Assets.Num());

		for (const FAssetData& Asset : Assets)
		{
			TSharedPtr<IAssetProcessor> Processor = GetProcessorForAsset(Asset);
			if (!Processor)
			{
				OnAssetResult(Asset, FAssetIndexingResult(), FString(TEXT("No processor")), EAssetIndexFailureReason::None);
				continue;
			}

			FCacheKey CacheKey = Private::FSemanticSearchBuildCacheTask::GenerateCacheKey(Asset, Processor.ToSharedRef());
			if (CacheKey == FCacheKey::Empty)
			{
				OnAssetResult(Asset, FAssetIndexingResult(), FString(TEXT("Invalid cache key")), EAssetIndexFailureReason::None);
				continue;
			}

			TSharedPtr<Private::FSemanticSearchBuildCacheTask> Task;
			BuildsJobs.FindOrProduceAndApply(
				CacheKey,
				[&Asset, &Processor, &CacheKey]() -> TSharedRef<Private::FSemanticSearchBuildCacheTask>
				{
					return Private::FSemanticSearchBuildCacheTask::Create(Asset, Processor.ToSharedRef(), FCacheKey(CacheKey)).ToSharedRef();
				},
				[&Task](const TSharedRef<Private::FSemanticSearchBuildCacheTask>& InTask)
				{
					Task = InTask;
				});

			Entries.Add({ FAssetData(Asset), Task, MoveTemp(CacheKey) });
		}

		Private::FSemanticSearchBuildCacheTask::BatchGetRecordData(MoveTemp(Entries),
			[OnAssetResult](const FAssetData& Asset, Private::FEmbeddingDerivedData&& EmbData,
				Private::FCaptionDerivedData&& CapData, FString&& Error, EAssetIndexFailureReason Reason)
			{
				FAssetIndexingResult Result;
				Result.Embedding = MoveTemp(EmbData.Embedding);
				Result.Caption = MoveTemp(CapData.Caption);
				Result.Keywords = MoveTemp(CapData.Keywords);
				OnAssetResult(Asset, MoveTemp(Result), MoveTemp(Error), Reason);
			}, bBuildOnMiss);
	}

	void FAssetProcessorManager::BatchGetQuantizedIndexingData(
		TConstArrayView<FAssetData> Assets,
		const FIoHash& CodebookHash,
		TFunction<void(const FAssetData& Asset, FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)> OnAssetResult,
		bool bBuildOnMiss)
	{
		using namespace DerivedData;

		TArray<Private::FBatchDDCEntry> Entries;
		Entries.Reserve(Assets.Num());

		for (const FAssetData& Asset : Assets)
		{
			TSharedPtr<IAssetProcessor> Processor = GetProcessorForAsset(Asset);
			if (!Processor)
			{
				OnAssetResult(Asset, FAssetIndexingResult(), FString(TEXT("No processor")), EAssetIndexFailureReason::None);
				continue;
			}

			FCacheKey CacheKey = Private::FSemanticSearchBuildCacheTask::GenerateCacheKey(Asset, Processor.ToSharedRef());
			if (CacheKey == FCacheKey::Empty)
			{
				OnAssetResult(Asset, FAssetIndexingResult(), FString(TEXT("Invalid cache key")), EAssetIndexFailureReason::None);
				continue;
			}

			TSharedPtr<Private::FSemanticSearchBuildCacheTask> Task;
			BuildsJobs.FindOrProduceAndApply(
				CacheKey,
				[&Asset, &Processor, &CacheKey]() -> TSharedRef<Private::FSemanticSearchBuildCacheTask>
				{
					return Private::FSemanticSearchBuildCacheTask::Create(Asset, Processor.ToSharedRef(), FCacheKey(CacheKey)).ToSharedRef();
				},
				[&Task](const TSharedRef<Private::FSemanticSearchBuildCacheTask>& InTask)
				{
					Task = InTask;
				});

			Entries.Add({ FAssetData(Asset), Task, MoveTemp(CacheKey) });
		}

		Private::FSemanticSearchBuildCacheTask::BatchGetQuantizedRecordData(MoveTemp(Entries), CodebookHash,
			[OnAssetResult](const FAssetData& Asset, Private::FCaptionDerivedData&& CapData,
				Private::FQuantizedEmbeddingDerivedData&& QData, FString&& Error, EAssetIndexFailureReason Reason)
			{
				FAssetIndexingResult Result;
				Result.Caption = MoveTemp(CapData.Caption);
				Result.Keywords = MoveTemp(CapData.Keywords);
				Result.QuantizedCodes = MoveTemp(QData.QuantizedEmbedding);
				OnAssetResult(Asset, MoveTemp(Result), MoveTemp(Error), Reason);
			}, bBuildOnMiss);
	}

	void FAssetProcessorManager::RegisterAssetProcessor(TSharedRef<IAssetProcessor>&& AssetProcessor)
	{
		UClass& AssetClass = AssetProcessor->GetSupportedClass();

		if (!ensureMsgf(AssetClass.HasAnyClassFlags(CLASS_Native), TEXT("The asset processor only support native asset type. The unreal is not really made to properly support non native asset types.")))
		{
			return;
		}

		const FTopLevelAssetPath ClassPath = AssetClass.GetClassPathName();
		
		TSharedPtr<IAssetProcessor> ExistingAssetProcessor;
		{
			FReadScopeLock ReadLock(AssetTypeToProcessorLock);
			ExistingAssetProcessor = AssetTypeToProcessor.FindRef(ClassPath);
		}

		if (ExistingAssetProcessor)
		{
			UnregisterAssetProcessor(AssetClass);
		}

		TSharedPtr<IAssetProcessor> ProcessorPtr = AssetProcessor;
		{
			FWriteScopeLock WriteLock(AssetTypeToProcessorLock);
			AssetTypeToProcessor.Add(ClassPath, MoveTemp(AssetProcessor));
		}

		PopulateChildClassesCache(AssetClass, ProcessorPtr);
	}

	void FAssetProcessorManager::UnregisterAssetProcessor(UClass& Class)
	{
		{
			FWriteScopeLock WriteLock(AssetTypeToProcessorLock);
			AssetTypeToProcessor.Remove(Class.GetClassPathName());
		}

		PopulateChildClassesCache(Class, nullptr);
	}

	void FAssetProcessorManager::PopulateChildClassesCache(UClass& Class, TSharedPtr<IAssetProcessor> Processor)
	{
		constexpr bool bRecursive = true;
		TArray<UClass*> DerivedClasses;
		GetDerivedClasses(&Class, DerivedClasses, bRecursive);

		FWriteScopeLock WriteLock(AssetTypeToProcessorLock);
		for (UClass* ChildClass : DerivedClasses)
		{
			if (ChildClass && ChildClass->HasAnyClassFlags(CLASS_Native))
			{
				if (Processor && Processor->SupportDerivedClasses())
				{
					AssetTypeToProcessor.Add(ChildClass->GetClassPathName(), Processor);
				}
				else
				{
					AssetTypeToProcessor.Remove(ChildClass->GetClassPathName());
				}
			}
		}
	}

	void FAssetProcessorManager::OnObjectsReinstanced(const TMap<UObject*, UObject*>& InReplacementObjects)
	{
		FWriteScopeLock WriteLock(AssetTypeToProcessorLock);
		TChunkedArray<TPair<FTopLevelAssetPath, TSharedPtr<IAssetProcessor>>> ProcessorsToAdd;
		for (auto It = AssetTypeToProcessor.CreateIterator(); It; ++It)
		{
			if (UObject* const* NewObject = InReplacementObjects.Find(FindObject<UClass>(nullptr, *It->Key.ToString())))
			{
				if (*NewObject)
				{
					if (const UClass* NewClass = Cast<const UClass>(*NewObject))
					{
						if (NewClass->HasAnyClassFlags(CLASS_Native))
						{
							ProcessorsToAdd.AddElement(TPair<FTopLevelAssetPath, TSharedPtr<IAssetProcessor>>(NewClass->GetClassPathName(), MoveTemp(It->Value)));
						}
					}
				}
				It.RemoveCurrent();
			}
		}

		AssetTypeToProcessor.Reserve(AssetTypeToProcessor.Num() + ProcessorsToAdd.Num());
		for (TPair<FTopLevelAssetPath, TSharedPtr<IAssetProcessor>>& Pair : ProcessorsToAdd)
		{
			AssetTypeToProcessor.Add(MoveTemp(Pair.Key), MoveTemp(Pair.Value));
		}
	}

	void FAssetProcessorManager::OnModuleUnloaded(TConstArrayView<UPackage*> UnloadingPackages)
	{
		FWriteScopeLock WriteLock(AssetTypeToProcessorLock);
		for (auto It = AssetTypeToProcessor.CreateIterator(); It ; ++It)
		{
			if (const UClass* Class = FindObject<UClass>(nullptr, *It->Key.ToString()))
			{
				if (UnloadingPackages.Contains(Class->GetPackage()))
				{
					It.RemoveCurrent();
				}
			}
		}
	}

	void FAssetProcessorManager::OnHotReloadComplete()
	{
		FWriteScopeLock WriteLock(AssetTypeToProcessorLock);
		for (auto It = AssetTypeToProcessor.CreateIterator(); It; ++It)
		{
			if (!FindObject<UClass>(nullptr, *It->Key.ToString()))
			{
				It.RemoveCurrent();
			}
		}
	}
}