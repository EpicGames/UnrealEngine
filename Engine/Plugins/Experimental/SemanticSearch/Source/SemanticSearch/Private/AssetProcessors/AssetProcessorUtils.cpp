// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetProcessors/AssetProcessorUtils.h"

#include "AssetProcessorManager.h"
#include "AssetProcessors/MaterialAssetProcessor.h"
#include "AssetProcessors/MaterialInstanceAssetProcessor.h"
#include "AssetProcessors/SkeletalMeshAssetProcessor.h"
#include "AssetProcessors/StaticMeshAssetProcessor.h"
#include "AssetProcessors/BlueprintAssetProcessor.h"
#include "AssetProcessors/TextureAssetProcessor.h"

#include "AssetRegistry/AssetData.h"
#include "Dom/JsonObject.h"
#include "UObject/Class.h"

namespace UE::SemanticSearch::Private
{

void SetMetadata(
	const TSharedPtr<FJsonObject>& Metadata,
	const TSharedRef<const FAssetData>& InAsset,
	FName AssetDataTag,
	FStringView CaptionMetadataKey)
{
	FString Value;
	if (InAsset->GetTagValue(AssetDataTag, Value))
	{
		Metadata->SetStringField(FString(CaptionMetadataKey), MoveTemp(Value));
	}
}

void SetMetadataWithDisplayString(
	const TSharedPtr<FJsonObject>& Metadata,
	const TSharedRef<const FAssetData>& InAsset,
	FName AssetDataTag,
	FStringView CaptionMetadataKey,
	const TMap<FString, FString>& DisplayMap)
{
	FString Value;
	if (InAsset->GetTagValue(AssetDataTag, Value))
	{
		const FString* DisplayValue = DisplayMap.Find(Value);
		Metadata->SetStringField(FString(CaptionMetadataKey), DisplayValue ? *DisplayValue : MoveTemp(Value));
	}
}

void PopulateFromEnum(TMap<FString, FString>& OutMap, const UEnum* Enum)
{
	// Reflection data is not thread safe in editor
	check(IsInGameThread());

	for (int32 Index = 0; Index < Enum->NumEnums() - 1; ++Index)
	{
		FString Key = Enum->GetNameStringByIndex(Index);

		// We don't want the user localization in our metadata
		FString DisplayValue = Enum->GetDisplayNameTextByIndex(Index).BuildSourceString();
		OutMap.Add(MoveTemp(Key), MoveTemp(DisplayValue));
	}
}

void RegisterDefaultAssetProcessors()
{
	UE::SemanticSearch::FAssetProcessorManager& Manager = UE::SemanticSearch::FAssetProcessorManager::Get();
	Manager.RegisterAssetProcessor<FMaterialProcessor>();
	Manager.RegisterAssetProcessor<FMaterialInstanceProcessor>();
	Manager.RegisterAssetProcessor<FStaticMeshProcessor>();
	Manager.RegisterAssetProcessor<FSkeletalMeshProcessor>();
	Manager.RegisterAssetProcessor<FTextureProcessor>();
	Manager.RegisterAssetProcessor<FBlueprintProcessor>();
}

}
