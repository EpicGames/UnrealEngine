// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FJsonObject;
struct FAssetData;
class UEnum;

namespace UE::SemanticSearch::Private
{

/**
 * Reads AssetDataTag from InAsset and, if present, writes the raw value into Metadata under CaptionMetadataKey.
 *
 * @param Metadata              JSON object to write the field into
 * @param InAsset               Asset whose tag store is queried
 * @param AssetDataTag          Tag name to look up on the asset
 * @param CaptionMetadataKey    Key used when writing the field into Metadata
 */
void SetMetadata(
	const TSharedPtr<FJsonObject>& Metadata,
	const TSharedRef<const FAssetData>& InAsset,
	FName AssetDataTag,
	FStringView CaptionMetadataKey);

/**
 * Reads AssetDataTag from InAsset and, if present, writes the value into Metadata under CaptionMetadataKey.
 * If DisplayMap contains an entry for the raw value, the mapped display string is written instead.
 *
 * @param Metadata              JSON object to write the field into
 * @param InAsset               Asset whose tag store is queried
 * @param AssetDataTag          Tag name to look up on the asset
 * @param CaptionMetadataKey    Key used when writing the field into Metadata
 * @param DisplayMap            Optional remapping from raw enum name strings to human-readable display strings
 */
void SetMetadataWithDisplayString(
	const TSharedPtr<FJsonObject>& Metadata,
	const TSharedRef<const FAssetData>& InAsset,
	FName AssetDataTag,
	FStringView CaptionMetadataKey,
	const TMap<FString, FString>& DisplayMap);

/**
 * Populates OutMap with name-to-display-string entries for every enumerator in Enum.
 * Must be called on the game thread as reflection data is not thread safe in the editor.
 *
 * @param OutMap    Map to populate with enumerator name to display string pairs
 * @param Enum      Enum whose enumerators are iterated
 */
void PopulateFromEnum(TMap<FString, FString>& OutMap, const UEnum* Enum);

/**
 * Register the build in asset processor that ship with this plugin
 */
void RegisterDefaultAssetProcessors();

}
