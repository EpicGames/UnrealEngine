// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetProcessors/BlueprintAssetProcessor.h"

#include "AssetProcessors/AssetProcessorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Blueprint/BlueprintSupport.h"
#include "Engine/Blueprint.h"

namespace UE::SemanticSearch::Private
{

UClass& FBlueprintProcessor::GetSupportedClass() const
{
	return *UBlueprint::StaticClass();
}

bool FBlueprintProcessor::SupportDerivedClasses() const
{
	return true;
}

TSharedPtr<FJsonObject> FBlueprintProcessor::GetMetadata(const TSharedRef<const FAssetData>& InAsset) const
{
	TSharedPtr<FJsonObject> Metadata = MakeShared<FJsonObject>();

	SetMetadata(Metadata, InAsset, FBlueprintTags::ParentClassPath, TEXTVIEW("Parent Class"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::NativeParentClassPath, TEXTVIEW("Native Parent Class"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::BlueprintType, TEXTVIEW("Blueprint Type"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::BlueprintDescription, TEXTVIEW("Description"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::BlueprintCategory, TEXTVIEW("Category"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::IsDataOnly, TEXTVIEW("Is Data Only"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::NumReplicatedProperties, TEXTVIEW("Replicated Properties"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::NumNativeComponents, TEXTVIEW("Native Components"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::NumBlueprintComponents, TEXTVIEW("Blueprint Components"));
	SetMetadata(Metadata, InAsset, FBlueprintTags::ImplementedInterfaces, TEXTVIEW("Implemented Interfaces"));

	return Metadata;
}

}
