// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Implementations/ThumbnailBaseProcessor.h"

namespace UE::SemanticSearch::Private
{

class FBlueprintProcessor : public FThumbnailBaseAssetProcessor
{
public:
	constexpr virtual FStringView GetProcessSubBucketName() const override
	{
		return TEXTVIEW("FBlueprintProcessor");
	}

	// Thread safe
	virtual UClass& GetSupportedClass() const override;
	virtual bool SupportDerivedClasses() const override;

	virtual TSharedPtr<FJsonObject> GetMetadata(const TSharedRef<const FAssetData>& InAsset) const override;
};

}
