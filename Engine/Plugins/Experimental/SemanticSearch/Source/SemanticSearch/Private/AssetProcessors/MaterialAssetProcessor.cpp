// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetProcessors/MaterialAssetProcessor.h"

#include "AssetProcessors/AssetProcessorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "UObject/ReflectedTypeAccessors.h"

namespace UE::SemanticSearch::Private
{

UClass& FMaterialProcessor::GetSupportedClass() const
{
	return *UMaterial::StaticClass();
}

bool FMaterialProcessor::SupportDerivedClasses() const
{
	return false;
}

FString FMaterialProcessor::DecodeShadingModelsBitmask(const FString& BitmaskStr) const
{
	// Expected asset registry format: "(ShadingModelField=4)"
	constexpr FStringView Prefix = TEXTVIEW("(ShadingModelField=");
	constexpr FStringView Suffix = TEXTVIEW(")");
	if (!ensureMsgf(BitmaskStr.StartsWith(Prefix) && BitmaskStr.EndsWith(Suffix),
		TEXT("FMaterialProcessor::DecodeShadingModelsBitmask: unexpected format '%s'"), *BitmaskStr))
	{
		return FString();
	}

	const FString NumberStr = BitmaskStr.Mid(Prefix.Len(), BitmaskStr.Len() - Prefix.Len() - 1);
	const int32 Bitmask = FCString::Atoi(*NumberStr);

	TArray<FString> ActiveModels;
	for (int32 Index = 0; Index < ShadingModelIndexToName.Num(); ++Index)
	{
		if (Bitmask & (1 << Index))
		{
			const FString& EnumName = ShadingModelIndexToName[Index];
			const FString* Display = MetaDataValueStringToDisplayString.Find(EnumName);
			ActiveModels.Add(Display ? *Display : EnumName);
		}
	}

	return FString::Join(ActiveModels, TEXT(", "));
}

TSharedPtr<FJsonObject> FMaterialProcessor::GetMetadata(const TSharedRef<const FAssetData>& InAsset) const
{
	TSharedPtr<FJsonObject> Metadata = MakeShared<FJsonObject>();

	static const FName MaterialDomainTag(TEXT("MaterialDomain"));
	SetMetadataWithDisplayString(Metadata, InAsset, MaterialDomainTag, TEXTVIEW("Material Domain"), MetaDataValueStringToDisplayString);

	static const FName BlendModeTag(TEXT("BlendMode"));
	SetMetadataWithDisplayString(Metadata, InAsset, BlendModeTag, TEXTVIEW("Blend Mode"), MetaDataValueStringToDisplayString);

	static const FName ShadingModelTag(TEXT("ShadingModel"));
	SetMetadataWithDisplayString(Metadata, InAsset, ShadingModelTag, TEXTVIEW("Shading Model"), MetaDataValueStringToDisplayString);

	static const FName ShadingModelsTag(TEXT("ShadingModels"));
	{
		FString Value;
		if (InAsset->GetTagValue(ShadingModelsTag, Value))
		{
			FString Decoded = DecodeShadingModelsBitmask(Value);
			if (!Decoded.IsEmpty())
			{
				Metadata->SetStringField(FString(TEXTVIEW("Shading Models")), MoveTemp(Decoded));
			}
		}
	}

	static const FName DecalResponseTag(TEXT("MaterialDecalResponse"));
	SetMetadataWithDisplayString(Metadata, InAsset, DecalResponseTag, TEXTVIEW("Decal Response"), MetaDataValueStringToDisplayString);

	static const FName TranslucencyLightingModeTag(TEXT("TranslucencyLightingMode"));
	SetMetadataWithDisplayString(Metadata, InAsset, TranslucencyLightingModeTag, TEXTVIEW("Translucency Lighting Mode"), MetaDataValueStringToDisplayString);

	static const FName HasSceneColorTag(TEXT("HasSceneColor"));
	SetMetadata(Metadata, InAsset, HasSceneColorTag, TEXTVIEW("Has Scene Color"));

	static const FName HasPerInstanceRandomTag(TEXT("HasPerInstanceRandom"));
	SetMetadata(Metadata, InAsset, HasPerInstanceRandomTag, TEXTVIEW("Has Per Instance Random"));

	static const FName HasPerInstanceCustomDataTag(TEXT("HasPerInstanceCustomData"));
	SetMetadata(Metadata, InAsset, HasPerInstanceCustomDataTag, TEXTVIEW("Has Per Instance Custom Data"));

	static const FName HasVertexInterpolatorTag(TEXT("HasVertexInterpolator"));
	SetMetadata(Metadata, InAsset, HasVertexInterpolatorTag, TEXTVIEW("Has Vertex Interpolator"));

	return Metadata;
}

FMaterialProcessor::FMaterialProcessor()
{
	check(IsInGameThread());

	const UEnum* ShadingModelEnum = StaticEnum<EMaterialShadingModel>();
	ShadingModelIndexToName.SetNum(MSM_NUM);
	for (int32 Index = 0; Index < MSM_NUM; ++Index)
	{
		ShadingModelIndexToName[Index] = ShadingModelEnum->GetNameStringByIndex(Index);
	}

	PopulateFromEnum(MetaDataValueStringToDisplayString, StaticEnum<EMaterialDomain>());
	PopulateFromEnum(MetaDataValueStringToDisplayString, StaticEnum<EBlendMode>());
	PopulateFromEnum(MetaDataValueStringToDisplayString, StaticEnum<EMaterialShadingModel>());
	PopulateFromEnum(MetaDataValueStringToDisplayString, StaticEnum<EMaterialDecalResponse>());
	PopulateFromEnum(MetaDataValueStringToDisplayString, StaticEnum<ETranslucencyLightingMode>());
}

}
