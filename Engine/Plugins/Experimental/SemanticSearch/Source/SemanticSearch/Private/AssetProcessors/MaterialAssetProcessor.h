// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Implementations/ThumbnailBaseProcessor.h"

namespace UE::SemanticSearch::Private
{

class FMaterialProcessor : public FThumbnailBaseAssetProcessor
{
public:
	// Must be created on the game thread
	FMaterialProcessor();

	constexpr virtual FStringView GetProcessSubBucketName() const override
	{
		return TEXTVIEW("FMaterialProcessor");
	}

	// Thread safe
	virtual UClass& GetSupportedClass() const override;
	virtual bool SupportDerivedClasses() const override;

	virtual TSharedPtr<FJsonObject> GetMetadata(const TSharedRef<const FAssetData>& InAsset) const override;

private:
	FString DecodeShadingModelsBitmask(const FString& BitmaskStr) const;

	// Cache the display values as those are not thread safe
	TMap<FString, FString> MetaDataValueStringToDisplayString;

	// Cache of EMaterialShadingModel index to name string — reflection data is not thread safe
	TArray<FString> ShadingModelIndexToName;
};

}
