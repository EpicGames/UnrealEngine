// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetProcessors/MaterialInstanceAssetProcessor.h"

#include "AssetProcessors/AssetProcessorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Materials/MaterialInstance.h"

namespace UE::SemanticSearch::Private
{

FMaterialInstanceProcessor::FMaterialInstanceProcessor() = default;

UClass& FMaterialInstanceProcessor::GetSupportedClass() const
{
	return *UMaterialInstance::StaticClass();
}

bool FMaterialInstanceProcessor::SupportDerivedClasses() const
{
	return true;
}

TSharedPtr<FJsonObject> FMaterialInstanceProcessor::GetMetadata(const TSharedRef<const FAssetData>& InAsset) const
{
	TSharedPtr<FJsonObject> Metadata = MakeShared<FJsonObject>();

	static const FName ParentTag(TEXT("Parent"));
	SetMetadata(Metadata, InAsset, ParentTag, TEXTVIEW("Parent"));

	static const FName HasSceneColorTag(TEXT("HasSceneColor"));
	SetMetadata(Metadata, InAsset, HasSceneColorTag, TEXTVIEW("Has Scene Color"));

	static const FName HasPerInstanceRandomTag(TEXT("HasPerInstanceRandom"));
	SetMetadata(Metadata, InAsset, HasPerInstanceRandomTag, TEXTVIEW("Has Per Instance Random"));

	static const FName HasPerInstanceCustomDataTag(TEXT("HasPerInstanceCustomData"));
	SetMetadata(Metadata, InAsset, HasPerInstanceCustomDataTag, TEXTVIEW("Has Per Instance Custom Data"));

	static const FName HasVertexInterpolatorTag(TEXT("HasVertexInterpolator"));
	SetMetadata(Metadata, InAsset, HasVertexInterpolatorTag, TEXTVIEW("Has Vertex Interpolator"));

	return Metadata;
}

}
