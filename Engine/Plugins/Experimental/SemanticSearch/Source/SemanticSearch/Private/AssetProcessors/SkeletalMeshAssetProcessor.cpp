// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetProcessors/SkeletalMeshAssetProcessor.h"

#include "AssetProcessors/AssetProcessorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Engine/SkeletalMesh.h"

namespace UE::SemanticSearch::Private
{

FSkeletalMeshProcessor::FSkeletalMeshProcessor() = default;

UClass& FSkeletalMeshProcessor::GetSupportedClass() const
{
	return *USkeletalMesh::StaticClass();
}

bool FSkeletalMeshProcessor::SupportDerivedClasses() const
{
	return false;
}

TSharedPtr<FJsonObject> FSkeletalMeshProcessor::GetMetadata(const TSharedRef<const FAssetData>& InAsset) const
{
	TSharedPtr<FJsonObject> Metadata = MakeShared<FJsonObject>();

	static const FName BonesTag(TEXT("Bones"));
	SetMetadata(Metadata, InAsset, BonesTag, TEXTVIEW("Bones"));

	static const FName MorphTargetsTag(TEXT("MorphTargets"));
	SetMetadata(Metadata, InAsset, MorphTargetsTag, TEXTVIEW("Morph Targets"));

	static const FName NaniteEnabledTag(TEXT("NaniteEnabled"));
	SetMetadata(Metadata, InAsset, NaniteEnabledTag, TEXTVIEW("Nanite Enabled"));

	static const FName NaniteTrianglesTag(TEXT("NaniteTriangles"));
	SetMetadata(Metadata, InAsset, NaniteTrianglesTag, TEXTVIEW("Nanite Triangles"));

	static const FName NaniteVerticesTag(TEXT("NaniteVertices"));
	SetMetadata(Metadata, InAsset, NaniteVerticesTag, TEXTVIEW("Nanite Vertices"));

	static const FName TrianglesTag(TEXT("Triangles"));
	SetMetadata(Metadata, InAsset, TrianglesTag, TEXTVIEW("Triangles"));

	static const FName VerticesTag(TEXT("Vertices"));
	SetMetadata(Metadata, InAsset, VerticesTag, TEXTVIEW("Vertices"));

	static const FName PhysicsAssetTag(TEXT("PhysicsAsset"));
	SetMetadata(Metadata, InAsset, PhysicsAssetTag, TEXTVIEW("Physics Asset"));

	static const FName SkeletonTag(TEXT("Skeleton"));
	SetMetadata(Metadata, InAsset, SkeletonTag, TEXTVIEW("Skeleton"));

	return Metadata;
}

}
