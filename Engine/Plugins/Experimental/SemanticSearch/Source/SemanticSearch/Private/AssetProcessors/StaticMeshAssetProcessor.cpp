// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetProcessors/StaticMeshAssetProcessor.h"

#include "AssetProcessors/AssetProcessorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Engine/StaticMesh.h"
#include "BodySetupEnums.h"
#include "UObject/ReflectedTypeAccessors.h"

namespace UE::SemanticSearch::Private
{

UClass& FStaticMeshProcessor::GetSupportedClass() const
{
	return *UStaticMesh::StaticClass();
}

bool FStaticMeshProcessor::SupportDerivedClasses() const
{
	return false;
}

TSharedPtr<FJsonObject> FStaticMeshProcessor::GetMetadata(const TSharedRef<const FAssetData>& InAsset) const
{
	TSharedPtr<FJsonObject> Metadata = MakeShared<FJsonObject>();

	static const FName CollisionComplexityTag(TEXT("CollisionComplexity"));
	SetMetadataWithDisplayString(Metadata, InAsset, CollisionComplexityTag, TEXTVIEW("Collision Complexity"), MetaDataValueStringToDisplayString);

	static const FName CollisionPrimsTag(TEXT("CollisionPrims"));
	SetMetadata(Metadata, InAsset, CollisionPrimsTag, TEXTVIEW("Collision Primitives"));

	static const FName DefaultCollisionTag(TEXT("DefaultCollision"));
	SetMetadata(Metadata, InAsset, DefaultCollisionTag, TEXTVIEW("Default Collision"));

	static const FName NaniteEnabledTag(TEXT("NaniteEnabled"));
	SetMetadata(Metadata, InAsset, NaniteEnabledTag, TEXTVIEW("Nanite Enabled"));

	static const FName NaniteTrianglesTag(TEXT("NaniteTriangles"));
	SetMetadata(Metadata, InAsset, NaniteTrianglesTag, TEXTVIEW("Nanite Triangles"));

	static const FName NaniteVerticesTag(TEXT("NaniteVertices"));
	SetMetadata(Metadata, InAsset, NaniteVerticesTag, TEXTVIEW("Nanite Vertices"));

	static const FName TrianglesTag(TEXT("Triangles"));
	SetMetadata(Metadata, InAsset, TrianglesTag, TEXTVIEW("Triangles"));

	static const FName VerticesTag(TEXT("Vertices"));
	SetMetadata(Metadata, InAsset, VerticesTag, TEXTVIEW("Vertices"));

	return Metadata;
}

FStaticMeshProcessor::FStaticMeshProcessor()
{
	PopulateFromEnum(MetaDataValueStringToDisplayString, StaticEnum<ECollisionTraceFlag>());
}

}
