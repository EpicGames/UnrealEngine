// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetProcessors/TextureAssetProcessor.h"

#include "AssetProcessors/AssetProcessorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Engine/Texture.h"
#include "Engine/TextureDefines.h"
#include "UObject/ReflectedTypeAccessors.h"

namespace UE::SemanticSearch::Private
{

UClass& FTextureProcessor::GetSupportedClass() const
{
	return *UTexture::StaticClass();
}

bool FTextureProcessor::SupportDerivedClasses() const
{
	return true;
}

TSharedPtr<FJsonObject> FTextureProcessor::GetMetadata(const TSharedRef<const FAssetData>& InAsset) const
{
	TSharedPtr<FJsonObject> Metadata = MakeShared<FJsonObject>();

	static const FName DimensionsTag(TEXT("Dimensions"));
	SetMetadata(Metadata, InAsset, DimensionsTag, TEXTVIEW("Dimensions"));

	static const FName MaxTextureSizeTag(TEXT("MaxTextureSize"));
	{
		FString Value;
		if (InAsset->GetTagValue(MaxTextureSizeTag, Value) && Value != TEXTVIEW("0"))
		{
			Metadata->SetStringField(FString(TEXTVIEW("MaxTextureSize")), MoveTemp(Value));
		}
	}

	static const FName WrappingX(TEXT("AddressX"));
	SetMetadataWithDisplayString(Metadata, InAsset, WrappingX, TEXTVIEW("X-axis Tiling Method"), MetaDataValueStringToDisplayString);

	static const FName WrappingY(TEXT("AddressY"));
	SetMetadataWithDisplayString(Metadata, InAsset, WrappingY, TEXTVIEW("Y-axis Tiling Method"), MetaDataValueStringToDisplayString);

	// Texture arrays
	static const FName WrappingZ(TEXT("AddressZ"));
	SetMetadataWithDisplayString(Metadata, InAsset, WrappingZ, TEXTVIEW("Z-axis Tiling Method"), MetaDataValueStringToDisplayString);

	// Volume texture
	static const FName WrappingMode(TEXT("AddressMode"));
	SetMetadataWithDisplayString(Metadata, InAsset, WrappingMode, TEXTVIEW("Tiling Method"), MetaDataValueStringToDisplayString);

	static const FName CompressionSettings(TEXT("CompressionSettings"));
	SetMetadataWithDisplayString(Metadata, InAsset, CompressionSettings, TEXTVIEW("Compression Settings"), MetaDataValueStringToDisplayString);

	static const FName SRGBTag(TEXT("SRGB"));
	SetMetadata(Metadata, InAsset, SRGBTag, TEXTVIEW("sRGB"));

	static const FName VirtualTextureStreamingTag(TEXT("VirtualTextureStreaming"));
	SetMetadata(Metadata, InAsset, VirtualTextureStreamingTag, TEXTVIEW("Virtual Texture Streaming"));

	return Metadata;
}

FTextureProcessor::FTextureProcessor()
{
	PopulateFromEnum(MetaDataValueStringToDisplayString, StaticEnum<TextureAddress>());
	PopulateFromEnum(MetaDataValueStringToDisplayString, StaticEnum<TextureCompressionSettings>());
}

}
