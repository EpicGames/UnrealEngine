// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Implementations/ThumbnailBaseProcessor.h"

namespace UE::SemanticSearch::Private
{

class FTextureProcessor : public FThumbnailBaseAssetProcessor
{
public:
	// Must be created on the game thread
	FTextureProcessor();

	constexpr virtual FStringView GetProcessSubBucketName() const override
	{
		return TEXTVIEW("FTextureProcessor");
	}

	// Thread safe
	virtual UClass& GetSupportedClass() const override;
	virtual bool SupportDerivedClasses() const override;

	virtual TSharedPtr<FJsonObject> GetMetadata(const TSharedRef<const FAssetData>& InAsset) const override;

private:
	// Cache the display value as those are not thread safe
	TMap<FString, FString> MetaDataValueStringToDisplayString;
};

}
