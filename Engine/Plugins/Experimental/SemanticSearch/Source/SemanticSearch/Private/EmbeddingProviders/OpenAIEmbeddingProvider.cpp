// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenAIEmbeddingProvider.h"

#include "Dom/JsonObject.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/Base64.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "SemanticSearchModule.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Settings/SemanticSearchSettings.h"

namespace UE::SemanticSearch::Private
{

static const TCHAR* GUEAssetTagAndCaptionSystemPrompt = TEXT(
	"You are a helpful assistant that produces both descriptive tags and a caption for an asset.\n"
	"\nYou may receive one or more of the following as input:\n\n"
	"- One or more images (thumbnails or screenshots of the asset)\n"
	"- Text content (descriptions, config files, material definitions, etc.)\n"
	"- Asset metadata as a JSON object (may include asset_type, asset_path, vertex_count, or other fields)\n\n"
	"Your goal is to:\n"
	"1. Generate up to 8 useful tags for the asset.\n"
	"2. Write a descriptive caption for the asset.\n\n"
	"When multiple images are provided, treat them as different views or aspects of the same asset "
	"and produce a single unified caption and tag set.\n\n"
	"When only text content is provided (no images), generate tags and a caption based on the text "
	"and metadata alone.\n\n"
	"Guidelines:\n"
	"- If images are present, they represent a 3D asset, but you must **not** mention that it is a "
	"3D model, render, or animation.\n"
	"- Ignore and do not include tags or descriptions related to:\n"
	"  - Backgrounds\n"
	"  - Camera angles or perspectives (e.g., \"front view\", \"side view\")\n"
	"  - Technical 3D terms (e.g., \"render\", \"texture\", \"polygon\", \"model\", \"animation\")\n"
	"- Base your tags and caption on the visible content of images and any provided text/metadata.\n"
	"- If you are unsure what the object is, leave the caption as an empty string (\"\"), but still "
	"provide any accurate descriptive tags (e.g., \"metallic\", \"circular\", \"mechanical\").\n"
	"- If the asset is a material, the tags/caption should be about the material (not the object "
	"itself like sphere or cube).\n"
	"- Tags should be 1-2 words each.\n"
	"- Include 1-3 color tags and some color info in caption where appropriate.\n"
	"- Asset metadata (path, type, etc.) is optionally provided. Use it as a hint, but if it "
	"contradicts the actual content, trust the content.\n"
	"- The tags and caption will be used for searching the asset, so keep that in mind to maximize "
	"usefulness."
);

uint32 FOpenAIConfig::GetHash() const
{
	uint32 Hash = FCrc::StrCrc32(*EmbeddingModel);
	Hash = HashCombine(Hash, static_cast<uint32>(EmbeddingDimension));
	// Combine with a provider hash name to avoid collision with other providers
	Hash = HashCombine(Hash, FCrc::StrCrc32(TEXT("openai")));
	return Hash;
}

/** Strip trailing '/' so we can safely append "/chat/completions" / "/embeddings". */
static FString JoinUrl(const FString& BaseUrl, const TCHAR* Path)
{
	FString Trimmed = BaseUrl;
	while (Trimmed.EndsWith(TEXT("/")))
	{
		Trimmed.LeftChopInline(1, EAllowShrinking::No);
	}
	return Trimmed + Path;
}

static FOpenAIConfig ReadConfigFromSettings()
{
	FOpenAIConfig OutConfig;
	if (const USemanticSearchSettings* Settings = USemanticSearchSettings::Get())
	{
		OutConfig.CaptioningBaseUrl  = Settings->CaptioningBaseUrl;
		OutConfig.CaptioningApiKey   = Settings->CaptioningApiKey;
		OutConfig.CaptioningModel    = Settings->CaptioningModel;
		OutConfig.EmbeddingBaseUrl   = Settings->EmbeddingBaseUrl;
		OutConfig.EmbeddingApiKey    = Settings->EmbeddingApiKey;
		OutConfig.EmbeddingModel     = Settings->EmbeddingModel;
		OutConfig.EmbeddingDimension = Settings->EmbeddingDimension;
	}
	return OutConfig;
}

FOpenAIEmbeddingProvider::FOpenAIEmbeddingProvider()
	: HttpClient(FSemanticHttpClient::CreateFromSettings())
{
	Config = ReadConfigFromSettings();
	const bool bMissingCaptionKey = Config.CaptioningApiKey.IsEmpty();
	const bool bMissingEmbeddingKey = Config.EmbeddingApiKey.IsEmpty();
	UE_LOGF(LogSemanticSearch, Log,
		"OpenAIEmbeddingProvider created (CaptionModel: %ls @ %ls, EmbedModel: %ls @ %ls, Dim: %d%ls%ls)",
		*Config.CaptioningModel, *Config.CaptioningBaseUrl,
		*Config.EmbeddingModel, *Config.EmbeddingBaseUrl,
		Config.EmbeddingDimension,
		bMissingCaptionKey ? TEXT(", CAPTION API KEY MISSING") : TEXT(""),
		bMissingEmbeddingKey ? TEXT(", EMBED API KEY MISSING") : TEXT(""));
}

int32 FOpenAIEmbeddingProvider::GetEmbeddingDimension() const
{
	return Config.EmbeddingDimension;
}

uint32 FOpenAIEmbeddingProvider::GetConfigHash() const
{
	return Config.GetHash();
}

// Build a single text-part object {"type":"text","text":...} for chat completions content arrays.
static TSharedRef<FJsonObject> MakeTextPart(const FString& Text)
{
	TSharedRef<FJsonObject> Part = MakeShared<FJsonObject>();
	Part->SetStringField(TEXT("type"), TEXT("text"));
	Part->SetStringField(TEXT("text"), Text);
	return Part;
}

// Build an image_url part with a base64 data URL.
static TSharedRef<FJsonObject> MakeImageUrlPart(const FAssetMedia& Media)
{
	const FString Base64 = FBase64::Encode(Media.Data);
	const FString DataUrl = FString::Printf(TEXT("data:%s;base64,%s"), *Media.MimeType, *Base64);

	TSharedRef<FJsonObject> ImageUrlObj = MakeShared<FJsonObject>();
	ImageUrlObj->SetStringField(TEXT("url"), DataUrl);
	ImageUrlObj->SetStringField(TEXT("detail"), TEXT("auto"));

	TSharedRef<FJsonObject> Part = MakeShared<FJsonObject>();
	Part->SetStringField(TEXT("type"), TEXT("image_url"));
	Part->SetObjectField(TEXT("image_url"), ImageUrlObj);
	return Part;
}

// Build the response_format object that constrains output to {tags: string[], caption: string}.
static TSharedRef<FJsonObject> MakeTagAndCaptionResponseFormat()
{
	TSharedRef<FJsonObject> TagsItems = MakeShared<FJsonObject>();
	TagsItems->SetStringField(TEXT("type"), TEXT("string"));

	TSharedRef<FJsonObject> TagsProp = MakeShared<FJsonObject>();
	TagsProp->SetStringField(TEXT("type"), TEXT("array"));
	TagsProp->SetObjectField(TEXT("items"), TagsItems);

	TSharedRef<FJsonObject> CaptionProp = MakeShared<FJsonObject>();
	CaptionProp->SetStringField(TEXT("type"), TEXT("string"));

	TSharedRef<FJsonObject> Properties = MakeShared<FJsonObject>();
	Properties->SetObjectField(TEXT("tags"), TagsProp);
	Properties->SetObjectField(TEXT("caption"), CaptionProp);

	TArray<TSharedPtr<FJsonValue>> Required;
	Required.Add(MakeShared<FJsonValueString>(TEXT("tags")));
	Required.Add(MakeShared<FJsonValueString>(TEXT("caption")));

	TSharedRef<FJsonObject> Schema = MakeShared<FJsonObject>();
	Schema->SetStringField(TEXT("type"), TEXT("object"));
	Schema->SetObjectField(TEXT("properties"), Properties);
	Schema->SetArrayField(TEXT("required"), Required);
	Schema->SetBoolField(TEXT("additionalProperties"), false);

	TSharedRef<FJsonObject> JsonSchema = MakeShared<FJsonObject>();
	JsonSchema->SetStringField(TEXT("name"), TEXT("TagAndCaptionOutput"));
	JsonSchema->SetBoolField(TEXT("strict"), true);
	JsonSchema->SetObjectField(TEXT("schema"), Schema);

	TSharedRef<FJsonObject> ResponseFormat = MakeShared<FJsonObject>();
	ResponseFormat->SetStringField(TEXT("type"), TEXT("json_schema"));
	ResponseFormat->SetObjectField(TEXT("json_schema"), JsonSchema);
	return ResponseFormat;
}

void FOpenAIEmbeddingProvider::GenerateCaptionAsync(
	FCaptionRequest&& Request,
	FOnCaptionComplete&& InOnComplete,
	DerivedData::FRequestOwner* RequestOwner)
{
	// Build the user-content list.
	// images -> image_url parts (base64 data URL); text/json -> text parts; metadata -> trailing text part.
	TArray<TSharedPtr<FJsonValue>> UserContent;

	for (const FAssetMedia& Media : Request.AssetMedia)
	{
		if (Media.Data.Num() == 0)
		{
			continue;
		}

		if (Media.MimeType.StartsWith(TEXT("image/")))
		{
			UserContent.Add(MakeShared<FJsonValueObject>(MakeImageUrlPart(Media)));
		}
		else if (Media.MimeType.StartsWith(TEXT("text/")) || Media.MimeType == TEXT("application/json"))
		{
			const FUTF8ToTCHAR Converter(reinterpret_cast<const ANSICHAR*>(Media.Data.GetData()), Media.Data.Num());
			const FString Text(Converter.Length(), Converter.Get());
			UserContent.Add(MakeShared<FJsonValueObject>(MakeTextPart(Text)));
		}
		else
		{
			UE_LOGF(LogSemanticSearch, Verbose, "Skipping unsupported media type: %ls", *Media.MimeType);
		}
	}

	// Append a metadata text part if the request carries metadata, asset_path, or asset_class.
	{
		TSharedPtr<FJsonObject> MetadataJson = Request.Metadata.IsValid()
			? MakeShared<FJsonObject>(*Request.Metadata)
			: MakeShared<FJsonObject>();

		if (!Request.AssetPath.IsEmpty())
		{
			MetadataJson->SetStringField(TEXT("asset_path"), Request.AssetPath);
		}
		if (!Request.AssetType.IsEmpty())
		{
			MetadataJson->SetStringField(TEXT("asset_class"), Request.AssetType);
		}

		if (MetadataJson->Values.Num() > 0)
		{
			FString MetaStr;
			TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> MetaWriter =
				TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&MetaStr);
			FJsonSerializer::Serialize(MetadataJson.ToSharedRef(), MetaWriter);
			UserContent.Add(MakeShared<FJsonValueObject>(
				MakeTextPart(FString::Printf(TEXT("Asset metadata: %s"), *MetaStr))));
		}
	}

	// Messages: [system, user(content_array)].
	TSharedRef<FJsonObject> SystemMessage = MakeShared<FJsonObject>();
	SystemMessage->SetStringField(TEXT("role"), TEXT("system"));
	SystemMessage->SetStringField(TEXT("content"), GUEAssetTagAndCaptionSystemPrompt);

	TSharedRef<FJsonObject> UserMessage = MakeShared<FJsonObject>();
	UserMessage->SetStringField(TEXT("role"), TEXT("user"));
	UserMessage->SetArrayField(TEXT("content"), UserContent);

	TArray<TSharedPtr<FJsonValue>> Messages;
	Messages.Add(MakeShared<FJsonValueObject>(SystemMessage));
	Messages.Add(MakeShared<FJsonValueObject>(UserMessage));

	TSharedRef<FJsonObject> Payload = MakeShared<FJsonObject>();
	Payload->SetStringField(TEXT("model"), Config.CaptioningModel);
	Payload->SetArrayField(TEXT("messages"), Messages);
	Payload->SetObjectField(TEXT("response_format"), MakeTagAndCaptionResponseFormat());

	FString PayloadStr;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer =
		TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&PayloadStr);
	FJsonSerializer::Serialize(Payload, Writer);

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Http = HttpClient.CreateRetryRequest();
	ConfigureJsonPostRequest(Http, JoinUrl(Config.CaptioningBaseUrl, TEXT("/chat/completions")));
	Http->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	Http->SetHeader(TEXT("Authorization"), FString::Printf(TEXT("Bearer %s"), *Config.CaptioningApiKey));
	Http->SetContentAsString(PayloadStr);

	DispatchHttpJsonRequest<FCaptionResponse>(Http, HttpClient.GetCaptionQueue(), RequestOwner, MoveTemp(InOnComplete),
		[](const TSharedRef<FJsonObject>& Json, FCaptionResponse& Response)
		{
			const TArray<TSharedPtr<FJsonValue>>* ChoicesArray = nullptr;
			if (!Json->TryGetArrayField(TEXT("choices"), ChoicesArray) || ChoicesArray->Num() == 0)
			{
				Response.ErrorMessage = TEXT("Missing 'choices' in response");
				return;
			}
			const TSharedPtr<FJsonObject> ChoiceObj = (*ChoicesArray)[0]->AsObject();
			if (!ChoiceObj.IsValid())
			{
				Response.ErrorMessage = TEXT("Invalid choice object in response");
				return;
			}

			// Surface refusal/truncation/safety failures so the caller can mark the asset failed
			// rather than silently indexing a half-formed response.
			FString FinishReason;
			if (ChoiceObj->TryGetStringField(TEXT("finish_reason"), FinishReason)
				&& FinishReason != TEXT("stop"))
			{
				Response.ErrorMessage = FString::Printf(TEXT("Non-stop finish_reason: %s"), *FinishReason);
				return;
			}

			const TSharedPtr<FJsonObject>* MessageObj = nullptr;
			if (!ChoiceObj->TryGetObjectField(TEXT("message"), MessageObj))
			{
				Response.ErrorMessage = TEXT("Missing 'message' in choice");
				return;
			}

			FString MessageContent;
			if (!(*MessageObj)->TryGetStringField(TEXT("content"), MessageContent) || MessageContent.IsEmpty())
			{
				// Some servers signal refusals here.
				FString Refusal;
				if ((*MessageObj)->TryGetStringField(TEXT("refusal"), Refusal) && !Refusal.IsEmpty())
				{
					Response.ErrorMessage = FString::Printf(TEXT("Model refused: %s"), *Refusal);
				}
				else
				{
					Response.ErrorMessage = TEXT("Empty 'content' in message");
				}
				return;
			}

			// content is a JSON string conforming to the schema we requested. Parse it.
			TSharedPtr<FJsonObject> StructuredJson;
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(MessageContent);
			if (!FJsonSerializer::Deserialize(Reader, StructuredJson) || !StructuredJson.IsValid())
			{
				Response.ErrorMessage = FString::Printf(TEXT("Failed to parse structured content: %s"),
					*MessageContent.Left(200));
				return;
			}

			StructuredJson->TryGetStringField(TEXT("caption"), Response.Caption);

			const TArray<TSharedPtr<FJsonValue>>* TagsArray = nullptr;
			if (StructuredJson->TryGetArrayField(TEXT("tags"), TagsArray))
			{
				for (const TSharedPtr<FJsonValue>& Val : *TagsArray)
				{
					FString Tag;
					if (Val->TryGetString(Tag))
					{
						Response.Keywords.Add(MoveTemp(Tag));
					}
				}
			}
		});
}

void FOpenAIEmbeddingProvider::GenerateEmbeddingAsync(
	const FStringView Text,
	FOnEmbeddingComplete&& InOnComplete,
	DerivedData::FRequestOwner* RequestOwner)
{
	TSharedRef<FJsonObject> Payload = MakeShared<FJsonObject>();
	Payload->SetStringField(TEXT("model"), Config.EmbeddingModel);
	Payload->SetStringField(TEXT("input"), FString(Text));
	Payload->SetNumberField(TEXT("dimensions"), Config.EmbeddingDimension);

	FString PayloadStr;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&PayloadStr);
	FJsonSerializer::Serialize(Payload, Writer);

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Http = HttpClient.CreateRetryRequest();
	ConfigureJsonPostRequest(Http, JoinUrl(Config.EmbeddingBaseUrl, TEXT("/embeddings")));
	Http->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	Http->SetHeader(TEXT("Authorization"), FString::Printf(TEXT("Bearer %s"), *Config.EmbeddingApiKey));
	Http->SetContentAsString(PayloadStr);

	DispatchHttpJsonRequest<FEmbeddingResponse>(Http, HttpClient.GetEmbedQueue(), RequestOwner, MoveTemp(InOnComplete),
		[](const TSharedRef<FJsonObject>& Json, FEmbeddingResponse& Response)
		{
			const TArray<TSharedPtr<FJsonValue>>* DataArray = nullptr;
			if (!Json->TryGetArrayField(TEXT("data"), DataArray) || DataArray->Num() == 0)
			{
				Response.ErrorMessage = TEXT("Missing 'data' in embeddings response");
				return;
			}
			const TSharedPtr<FJsonObject> Item = (*DataArray)[0]->AsObject();
			if (!Item.IsValid())
			{
				Response.ErrorMessage = TEXT("Invalid embedding item");
				return;
			}

			const TArray<TSharedPtr<FJsonValue>>* EmbArray = nullptr;
			if (!Item->TryGetArrayField(TEXT("embedding"), EmbArray))
			{
				Response.ErrorMessage = TEXT("Missing 'embedding' in data[0]");
				return;
			}

			Response.Embedding.Reserve(EmbArray->Num());
			for (const TSharedPtr<FJsonValue>& Val : *EmbArray)
			{
				double Num;
				if (Val->TryGetNumber(Num))
				{
					Response.Embedding.Add(static_cast<float>(Num));
				}
				else
				{
					Response.Embedding.Empty();
					Response.ErrorMessage = TEXT("Invalid value in embedding array");
					return;
				}
			}

			Json->TryGetStringField(TEXT("model"), Response.ModelVersion);
		});
}

void FOpenAIEmbeddingProvider::CancelAllPendingRequests()
{
	HttpClient.CancelAllPendingRequests();
}

}
