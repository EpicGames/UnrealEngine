// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IEmbeddingProvider.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "SemanticHttpUtils.h"

namespace UE::SemanticSearch::Private
{

/**
 * Configuration for the OpenAI provider, snapshotted from USemanticSearchSettings at construction.
 * Snapshotting (not live-reading) keeps GetConfigHash() stable across the provider's lifetime,
 * which the DDC cache key path depends on.
 */
struct FOpenAIConfig
{
	FString CaptioningBaseUrl;
	FString CaptioningApiKey;
	FString CaptioningModel;
	FString EmbeddingBaseUrl;
	FString EmbeddingApiKey;
	FString EmbeddingModel;
	int32 EmbeddingDimension = 512;

	/** Hash of the embedding model + dimension. Affects DDC cache key. */
	uint32 GetHash() const;
};

/**
 *  OpenAI-compatible embedding provider. Calls /chat/completions for captioning
 *  (with structured JSON output enforcing {tags, caption}) and /embeddings for embeddings.
 *
 *  Works against any OpenAI-compatible server (api.openai.com, Google/Anthropic OpenAI compatibitle endpoints,
 *  Ollama, LiteLLM).
 */
class FOpenAIEmbeddingProvider : public IEmbeddingProvider
{
public:
	FOpenAIEmbeddingProvider();

	virtual int32 GetEmbeddingDimension() const override;
	virtual void GenerateCaptionAsync(FCaptionRequest&& Request, FOnCaptionComplete&& OnComplete, DerivedData::FRequestOwner* RequestOwner) override;
	virtual void GenerateEmbeddingAsync(const FStringView Text, FOnEmbeddingComplete&& OnComplete, DerivedData::FRequestOwner* RequestOwner) override;

	uint32 GetConfigHash() const override;
	void CancelAllPendingRequests() override;

private:
	FOpenAIConfig Config;
	FSemanticHttpClient HttpClient;
};

}
