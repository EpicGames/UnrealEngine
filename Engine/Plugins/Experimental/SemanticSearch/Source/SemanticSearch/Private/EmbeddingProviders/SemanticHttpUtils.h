// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "DerivedDataRequest.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataRequestTypes.h"
#include "Dom/JsonObject.h"
#include "HttpRetrySystem.h"
#include "HybridSearchIndex.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "SemanticSearchModule.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Settings/SemanticSearchSettings.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include <atomic>

namespace UE::SemanticSearch::Private
{

/**
 * Async-friendly concurrency limiter for HTTP requests.
 * Limits the number of in-flight requests and queues excess for later dispatch.
 */
struct FThrottledRequestQueue
{
	FCriticalSection Lock;
	TQueue<TFunction<void()>> PendingRequests;
	int32 InFlightCount = 0;
	int32 MaxConcurrent = 8;

	/** Try to dispatch a request. If at capacity, queues it for later. */
	inline void Enqueue(TFunction<void()>&& DispatchFn)
	{
		FScopeLock ScopeLock(&Lock);
		if (InFlightCount < MaxConcurrent)
		{
			++InFlightCount;
			DispatchFn();
		}
		else
		{
			PendingRequests.Enqueue(MoveTemp(DispatchFn));
		}
	}

	/** Called when a request completes. Dispatches next queued request if any. */
	inline void OnRequestComplete()
	{
		TFunction<void()> Next;
		{
			FScopeLock ScopeLock(&Lock);
			if (!PendingRequests.Dequeue(Next))
			{
				--InFlightCount;
			}
			// else: InFlightCount stays the same — one finished, one starting
		}
		if (Next)
		{
			Next();
		}
	}

	/** Clear all pending (not yet dispatched) requests. In-flight requests are not affected. */
	inline void CancelPending()
	{
		FScopeLock ScopeLock(&Lock);
		PendingRequests.Empty();
		UE_LOGF(LogSemanticSearch, Log, "Cancelled pending HTTP requests (InFlight: %d)", InFlightCount);
	}
};

/**
 * Shared HTTP plumbing for embedding providers: owns the FHttpRetrySystem manager and the
 * two throttled queues (caption + embed), and exposes a CreateRetryRequest() factory that
 * applies the standard retry policy (retry on 429/5xx, POST only).
 *
 * Providers compose this instead of duplicating retry/queue/concurrency wiring.
 */
class FSemanticHttpClient
{
public:
	FSemanticHttpClient(int32 MaxRetries, double RetryTimeoutSeconds, int32 MaxConcurrentCaption, int32 MaxConcurrentEmbed)
		: RetryManager(MakeShared<FHttpRetrySystem::FManager>(
			FHttpRetrySystem::RetryLimitCount(MaxRetries),
			FHttpRetrySystem::RetryTimeoutRelativeSeconds(RetryTimeoutSeconds)))
	{
		CaptionQueue.MaxConcurrent = MaxConcurrentCaption;
		EmbedQueue.MaxConcurrent = MaxConcurrentEmbed;
	}

	/** Build a client snapshotting retry/concurrency knobs from USemanticSearchSettings. */
	static FSemanticHttpClient CreateFromSettings()
	{
		const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
		const int32 MaxRetries         = Settings ? Settings->MaxRetries                    : 1;
		const float RetryTimeoutSecs   = Settings ? Settings->RetryTimeoutSeconds           : 300.0f;
		const int32 MaxCaptionInFlight = Settings ? Settings->MaxConcurrentCaptionRequests  : 2000;
		const int32 MaxEmbedInFlight   = Settings ? Settings->MaxConcurrentEmbedRequests    : 2000;
		return FSemanticHttpClient(MaxRetries, static_cast<double>(RetryTimeoutSecs), MaxCaptionInFlight, MaxEmbedInFlight);
	}

	/** Create a retry-aware HTTP request (retries on 429/5xx, POST only). */
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateRetryRequest() const
	{
		FHttpRetrySystem::FRetryResponseCodes RetryResponseCodes({429, 500, 502, 503, 504});
		FHttpRetrySystem::FRetryVerbs RetryVerbs;
		RetryVerbs.Add(FName(TEXT("POST")));

		return RetryManager->CreateRequest(
			FHttpRetrySystem::FRetryLimitCountSetting(),
			FHttpRetrySystem::FRetryTimeoutRelativeSecondsSetting(),
			RetryResponseCodes,
			RetryVerbs);
	}

	FThrottledRequestQueue& GetCaptionQueue() { return CaptionQueue; }
	FThrottledRequestQueue& GetEmbedQueue()   { return EmbedQueue; }

	/** Clear pending (un-dispatched) requests in both queues. In-flight requests are unaffected. */
	void CancelAllPendingRequests()
	{
		CaptionQueue.CancelPending();
		EmbedQueue.CancelPending();
	}

private:
	TSharedRef<FHttpRetrySystem::FManager> RetryManager;
	FThrottledRequestQueue CaptionQueue;
	FThrottledRequestQueue EmbedQueue;
};

/**
 * Wraps an IHttpRequest as a DerivedData::FRequest so it can be tracked / canceled
 * by the DDC subsystem. Mirrors the priority and cancel signals from the owner
 * down to the underlying HTTP request.
 */
class FSemanticHttpRequest : public DerivedData::FRequestBase
{
public:
	FSemanticHttpRequest(
		const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& InHttpRequest,
		DerivedData::FRequestOwner& InRequestOwner,
		TSharedRef<std::atomic<bool>, ESPMode::ThreadSafe> InCanceledBeforeSendFlag)
		: FRequestBase()
		, HttpRequest(InHttpRequest)
		, RequestOwner(&static_cast<DerivedData::IRequestOwner&>(InRequestOwner))
		, CanceledBeforeSendFlag(MoveTemp(InCanceledBeforeSendFlag))
	{
		RequestCompleteDelegate = MoveTemp(HttpRequest->OnProcessRequestComplete());
		HttpRequest->OnProcessRequestComplete().BindRaw(this, &FSemanticHttpRequest::OnRequestComplete);
		RequestOwner->Begin(this);
	}

	/**
	 * Start the HTTP request. Separated from construction so Begin() can run in the caller's DDC context.
	 * Returns true if HTTP was dispatched, false if the request was canceled before it could be sent
	 * (in which case the caller must release the throttled queue slot).
	 */
	bool Start()
	{
		EState Expected = EState::Pending;
		if (State.compare_exchange_strong(Expected, EState::Started))
		{
			HttpRequest->ProcessRequest();
			return true;
		}
		return false;
	}

	virtual ~FSemanticHttpRequest()
	{
		HttpRequest->OnProcessRequestComplete().Unbind();
	}

	virtual void SetPriority(DerivedData::EPriority Priority) override
	{
		// Keep the priority lower the then actual tasks to not accidentally starve the ddc
		EHttpRequestPriority HttpPriority = EHttpRequestPriority::Lowest;
		switch (Priority)
		{
			case DerivedData::EPriority::Lowest:
				HttpPriority = EHttpRequestPriority::Lowest;
				break;
			case DerivedData::EPriority::Low:
				HttpPriority = EHttpRequestPriority::Lowest;
				break;
			case DerivedData::EPriority::Normal:
				HttpPriority = EHttpRequestPriority::Low;
				break;
			case DerivedData::EPriority::High:
				HttpPriority = EHttpRequestPriority::Normal;
				break;
			case DerivedData::EPriority::Highest:
				HttpPriority = EHttpRequestPriority::High;
			case DerivedData::EPriority::Blocking:
				HttpPriority = EHttpRequestPriority::Highest;
				break;

			default:
				break;
		}

		HttpRequest->SetPriority(HttpPriority);
	}


	virtual void Cancel() override
	{
		EState Expected = EState::Pending;
		if (State.compare_exchange_strong(Expected, EState::Canceled))
		{
			// Cancel() raced ahead of Start(); Start() will no-op when it runs.
			// Avoids the slow HttpRequest->CancelRequest() path on an unsent request.
			// Signal downstream and synthesize a failure completion so the caller's
			// OnComplete callback still fires with a canceled error.
			CanceledBeforeSendFlag->store(true, std::memory_order_release);
			OnRequestComplete(HttpRequest, nullptr, false);
			return;
		}
		if (Expected == EState::Started)
		{
			HttpRequest->CancelRequest();
		}
		// else: already canceled or completed, nothing to do.
	}

	virtual void Wait() override
	{
		HttpRequest->ProcessRequestUntilComplete();
	}

private:
	void OnRequestComplete(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnected)
	{
		// Mark completed so a late Cancel() skips the redundant HttpRequest->CancelRequest().
		State.store(EState::Completed, std::memory_order_release);

		// This will die after the scope
		RequestOwner->End(this, [this, &Request, &Response, bConnected]()
			{
				// Free the http request thread asap
				RequestOwner->LaunchTask(TEXT("SemanticSearch_RemoteRequestComplete"), [InRequestCompleteDelegate = MoveTemp(RequestCompleteDelegate), Request, Response, bConnected]()
					{
						InRequestCompleteDelegate.ExecuteIfBound(Request, Response, bConnected);
					});
			});
	}

	enum class EState : uint8 { Pending, Started, Canceled, Completed };

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest;
	DerivedData::IRequestOwner* RequestOwner;
	FHttpRequestCompleteDelegate RequestCompleteDelegate;
	TSharedRef<std::atomic<bool>, ESPMode::ThreadSafe> CanceledBeforeSendFlag;
	std::atomic<EState> State { EState::Pending };
};

/**
 * Validate an HTTP/JSON response and populate the standard error cases.
 * On success, invokes ParseBody(Json, Response) to fill in response-specific fields.
 * ResponseT must expose an FString ErrorMessage member.
 */
template <typename ResponseT, typename BodyParserT>
inline ResponseT ParseHttpJsonResponse(FHttpResponsePtr Resp, bool bConnected, BodyParserT&& ParseBody)
{
	ResponseT Response;
	if (!bConnected || !Resp.IsValid())
	{
		Response.ErrorMessage = TEXT("Connection failed");
		Response.FailureReason = EAssetIndexFailureReason::Provider;
		return Response;
	}
	if (Resp->GetResponseCode() != 200)
	{
		Response.ErrorMessage = FString::Printf(TEXT("HTTP %d: %s"),
			Resp->GetResponseCode(), *Resp->GetContentAsString().Left(200));
		Response.FailureReason = EAssetIndexFailureReason::Provider;
		return Response;
	}
	TSharedPtr<FJsonObject> Json;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Resp->GetContentAsString());
	if (!FJsonSerializer::Deserialize(Reader, Json) || !Json.IsValid())
	{
		Response.ErrorMessage = TEXT("Failed to parse JSON response");
		Response.FailureReason = EAssetIndexFailureReason::Provider;
		return Response;
	}
	FString ServerError;
	if (Json->TryGetStringField(TEXT("error"), ServerError))
	{
		Response.ErrorMessage = FString::Printf(TEXT("Server error: %s"), *ServerError);
		Response.FailureReason = EAssetIndexFailureReason::Provider;
		return Response;
	}
	ParseBody(Json.ToSharedRef(), Response);
	// If the body parser set ErrorMessage without a reason, default to Provider.
	if (!Response.ErrorMessage.IsEmpty() && Response.FailureReason == EAssetIndexFailureReason::None)
	{
		Response.FailureReason = EAssetIndexFailureReason::Provider;
	}
	return Response;
}

/**
 * Bind the cancel-aware completion delegate on Http, then enqueue the request through Queue.
 * When RequestOwner is provided, wraps the request as an FSemanticHttpRequest so DDC can cancel it;
 * when null, dispatches directly through the queue.
 */
template <typename ResponseT, typename OnCompleteT, typename BodyParserT>
inline void DispatchHttpJsonRequest(
	const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& Http,
	FThrottledRequestQueue& Queue,
	DerivedData::FRequestOwner* RequestOwner,
	OnCompleteT&& InOnComplete,
	BodyParserT&& ParseBody)
{
	// Short-circuit when the owner has already been canceled.
	if (RequestOwner && RequestOwner->IsCanceled())
	{
		ResponseT Response;
		Response.ErrorMessage = TEXT("Request canceled");
		InOnComplete(MoveTemp(Response));
		return;
	}

	TSharedRef<std::atomic<bool>, ESPMode::ThreadSafe> CanceledBeforeSendFlag =
		MakeShared<std::atomic<bool>, ESPMode::ThreadSafe>(false);

	auto OnComplete = MakeShared<OnCompleteT>(MoveTemp(InOnComplete));

	Http->OnProcessRequestComplete().BindLambda(
		[&Queue, OnComplete, CanceledBeforeSendFlag, ParseBody = Forward<BodyParserT>(ParseBody)]
		(FHttpRequestPtr /*Req*/, FHttpResponsePtr Resp, bool bConnected)
		{
			if (CanceledBeforeSendFlag->load(std::memory_order_acquire))
			{
				// Request was canceled before it hit the wire. The throttled queue slot
				// is released separately by the enqueued dispatch lambda when Start() no-ops.
				// Reason stays None — cancellation isn't a failure to mark; the module
				// callback skips MarkFailed when Reason == None.
				ResponseT Response;
				Response.ErrorMessage = TEXT("Request canceled");
				(*OnComplete)(MoveTemp(Response));
				return;
			}

			Queue.OnRequestComplete();
			(*OnComplete)(ParseHttpJsonResponse<ResponseT>(Resp, bConnected, ParseBody));
		});

	if (RequestOwner)
	{
		// Create DDC request wrapper immediately so Begin() runs in the caller's barrier context.
		TRefCountPtr<FSemanticHttpRequest> DdcRequest =
			new FSemanticHttpRequest(Http, *RequestOwner, CanceledBeforeSendFlag);
		Queue.Enqueue([&Queue, DdcRequest]()
		{
			if (!DdcRequest->Start())
			{
				// Start was a no-op because Cancel() already fired; release the slot
				// the queue handed to us so pending requests can dispatch.
				Queue.OnRequestComplete();
			}
		});
	}
	else
	{
		Queue.Enqueue([Http]() { Http->ProcessRequest(); });
	}
}

/** Apply POST/timeout/thread-policy settings common to every request. */
inline void ConfigureJsonPostRequest(const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& Http, const FString& URL)
{
	Http->SetURL(URL);
	Http->SetVerb(TEXT("POST"));
	Http->SetTimeout(120.0f);
	Http->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);
}

}
