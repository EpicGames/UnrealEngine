// Copyright Epic Games, Inc. All Rights Reserved.

#include "HybridSearchIndex.h"

#include "SemanticSearchModule.h"
#include "AssetRegistry/AssetData.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "Tasks/Task.h"
#include "Tasks/Pipe.h"
#include "Hash/Blake3.h"
#include "Implementations/SemanticSearchImplementationUtils.h"
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IQuantizedVectorIndex.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Settings/SemanticSearchSettings.h"
#include "TextIndex/BM25Index.h"
#include "VectorIndex/VectorIndexFactory.h"

#include "Containers/HashTable.h"
#include "Containers/Queue.h"
#include "Misc/TVariant.h"

#include <atomic>

namespace UE::SemanticSearch
{

// --- Command queue internals ---

struct FMutationAdd
{
	FAssetData Asset;
	TArray<float> Embedding;
	FString Caption;
	TArray<FString> Keywords;
	uint32 IndexId;
};

struct FMutationAddQuantized
{
	FAssetData Asset;
	TArray<uint8> QuantizedCodes;
	FString Caption;
	TArray<FString> Keywords;
	uint32 IndexId;
};

struct FMutationMarkFailed
{
	int64 AssetId;
	EAssetIndexFailureReason Reason;
	uint32 IndexId;
};

struct FMutationClearFailed
{
	int64 AssetId;
};

struct FMutationPurgePreProcessorFailedNotIn
{
	TSet<int64> ValidIDs;
};

struct FMutationRemoveAsset
{
	FAssetData Asset;
};

struct FMutationRemovePath
{
	FString AssetPath;
};

struct FMutationRemoveId
{
	int64 AssetId;
};

struct FMutationSetVectorIndex
{
	TSharedPtr<IVectorIndex> NewVectorIndex;
};

struct FMutationClearBM25 {};

struct FMutationResetBM25AndFailed {};

struct FMutationSave
{
	FString FilePath;
	FString PreProcessorFailedPath;
	TMap<int64, FIoHash> PackageHashes;
};

using FPendingMutation = TVariant<
	FMutationAdd,
	FMutationAddQuantized,
	FMutationMarkFailed,
	FMutationClearFailed,
	FMutationPurgePreProcessorFailedNotIn,
	FMutationRemoveAsset,
	FMutationRemovePath,
	FMutationRemoveId,
	FMutationSetVectorIndex,
	FMutationClearBM25,
	FMutationResetBM25AndFailed,
	FMutationSave>;

struct FPendingSearch
{
	FString QueryText;
	TArray<float> QueryEmbedding;
	int32 K = 0;
	TArray<int64> IDFilter;
	float DistanceCutoff = TNumericLimits<float>::Max();
	TFunction<void(TArray<FHybridSearchResult>&&)> Callback;
};

struct FPendingQuantize
{
	TArray<float> Embedding;
	TFunction<void(TArray<uint8>&&, bool bSuccess)> Callback;
};

struct FPendingContains
{
	TArray<int64> IDs;
	TFunction<void(TSet<int64>&&)> Callback;
};

struct FPendingExtractTraining
{
	TArray<int64> CandidateIDs;
	TFunction<void(TArray<int64>&&, TArray<float>&&, int32)> Callback;
};

struct FPendingIsFailed
{
	TArray<int64> IDs;
	TFunction<void(TSet<int64>&&)> Callback;
};

struct FPendingIsRetryableFailed
{
	TArray<int64> IDs;
	TFunction<void(TSet<int64>&&)> Callback;
};

using FPendingQuery = TVariant<
	FPendingSearch,
	FPendingQuantize,
	FPendingContains,
	FPendingExtractTraining,
	FPendingIsFailed,
	FPendingIsRetryableFailed>;

/** Immutable snapshot of index stats. Consumer thread builds a new instance and swaps the pointer. */
struct FCachedIndexStats
{
	int32 VectorCount = 0;
	int32 BM25Count = 0;
	int32 Dimension = 0;
	int64 EstimatedVectorMemoryBytes = 0;
	int64 EstimatedBM25MemoryBytes = 0;
	bool bIsTrained = false;
	bool bSupportsQuantization = false;
	bool bIsInitialized = false;
	int32 FailedCount = 0;
	int32 PreProcessorFailedCount = 0;
	FIoHash CodebookHash;
};

struct FCommandQueueData
{
	TQueue<FPendingMutation, EQueueMode::Mpsc> PendingMutations;
	TQueue<FPendingQuery, EQueueMode::Mpsc> PendingQueries;
	std::atomic<bool> bTaskInFlight{false};
	std::atomic<bool> bShutdownRequested{false};
	std::atomic<bool> bDiscardMutations{false};
	std::atomic<bool> bPriorityBoostRequested{false};
	mutable FCriticalSection CachedStatsLock;
	TSharedPtr<const FCachedIndexStats, ESPMode::ThreadSafe> CachedStats;
	UE::Tasks::FPipe SavePipe{ TEXT("HybridSearchSavePipe") };
};

// --- End command queue internals ---

struct FHybridIndexHeader
{
	static constexpr uint32 Magic = 0x48594258;   // "HYBX"
	static constexpr uint32 Version = 1;
};

FHybridSearchIndex& FHybridSearchIndex::Get()
{
	static FHybridSearchIndex Instance;
	return Instance;
}

FHybridSearchIndex::FHybridSearchIndex()
	: BM25Index(MakeUnique<FBM25Index>())
	, QueueData(MakeUnique<FCommandQueueData>())
{
}

FHybridSearchIndex::~FHybridSearchIndex()
{
	StopCommandQueue();
}

// --- Command queue methods ---

void FHybridSearchIndex::StartCommandQueue()
{
	QueueData->bShutdownRequested.store(false, std::memory_order_release);
}

void FHybridSearchIndex::StopCommandQueue()
{
	QueueData->bShutdownRequested.store(true, std::memory_order_release);
	while (QueueData->bTaskInFlight.load(std::memory_order_acquire))
	{
		FPlatformProcess::YieldThread();
	}
	// Worker has drained (sole producer into SavePipe), so any remaining save
	// tasks are already launched. Flush them before ~FPipe runs.
	QueueData->SavePipe.WaitUntilEmpty();
}

void FHybridSearchIndex::EnsureProcessingTask(UE::Tasks::ETaskPriority Priority)
{
	if (QueueData->bShutdownRequested.load(std::memory_order_acquire) || IsEngineExitRequested())
	{
		return;
	}

	bool bExpected = false;
	if (QueueData->bTaskInFlight.compare_exchange_strong(bExpected, true, std::memory_order_acq_rel))
	{
		UE::Tasks::Launch(TEXT("HybridSearchProcess"),
			[this]() { ProcessCommandBatch(); },
			Priority);
	}
	else if (Priority < UE::Tasks::ETaskPriority::BackgroundNormal)
	{
		// Task already in flight at lower priority  - request a boosted re-launch after it completes
		QueueData->bPriorityBoostRequested.store(true, std::memory_order_release);
	}
}

void FHybridSearchIndex::Add(FAssetData Asset, TArray<float> Embedding,
	FString Caption, TArray<FString> Keywords, uint32 InIndexId)
{
	// Short-circuit stale callbacks (captured IndexId from before a CancelIndexing / switch).
	// The consumer re-checks as a backstop for the tiny producer/consumer race window.
	if (InIndexId != GetIndexId())
	{
		return;
	}
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationAdd>{},
			MoveTemp(Asset), MoveTemp(Embedding), MoveTemp(Caption), MoveTemp(Keywords), InIndexId));
	EnsureProcessingTask();
}

void FHybridSearchIndex::AddQuantized(FAssetData Asset, TArray<uint8> QuantizedCodes,
	FString Caption, TArray<FString> Keywords, uint32 InIndexId)
{
	if (InIndexId != GetIndexId())
	{
		return;
	}
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationAddQuantized>{},
			MoveTemp(Asset), MoveTemp(QuantizedCodes), MoveTemp(Caption), MoveTemp(Keywords), InIndexId));
	EnsureProcessingTask();
}

void FHybridSearchIndex::MarkFailed(int64 AssetId, EAssetIndexFailureReason Reason, uint32 InIndexId)
{
	if (InIndexId != GetIndexId())
	{
		return;
	}
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationMarkFailed>{}, AssetId, Reason, InIndexId));
	EnsureProcessingTask();
}

void FHybridSearchIndex::ClearFailedState(int64 AssetId)
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationClearFailed>{}, AssetId));
	EnsureProcessingTask();
}

void FHybridSearchIndex::PurgePreProcessorFailedNotIn(TSet<int64> ValidIDs)
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationPurgePreProcessorFailedNotIn>{}, MoveTemp(ValidIDs)));
	EnsureProcessingTask();
}

void FHybridSearchIndex::Remove(FAssetData Asset)
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationRemoveAsset>{}, MoveTemp(Asset)));
	EnsureProcessingTask();
}

void FHybridSearchIndex::RemoveByPath(FString AssetPath)
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationRemovePath>{}, MoveTemp(AssetPath)));
	EnsureProcessingTask();
}

void FHybridSearchIndex::RemoveById(int64 AssetId)
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationRemoveId>{}, AssetId));
	EnsureProcessingTask();
}

void FHybridSearchIndex::SetVectorIndex(TSharedPtr<IVectorIndex> NewIndex)
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationSetVectorIndex>{}, MoveTemp(NewIndex)));
	EnsureProcessingTask();
}

void FHybridSearchIndex::ClearBM25()
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationClearBM25>{}));
	EnsureProcessingTask();
}

void FHybridSearchIndex::Save(FString FilePath, FString PreProcessorFailedPath, TMap<int64, FIoHash> InPackageHashes)
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationSave>{},
			MoveTemp(FilePath), MoveTemp(PreProcessorFailedPath), MoveTemp(InPackageHashes)));
	EnsureProcessingTask();
}

void FHybridSearchIndex::SearchAsync(FString QueryText, TArray<float> QueryEmbedding,
	int32 K, TArray<int64> IDFilter, float DistanceCutoff,
	TFunction<void(TArray<FHybridSearchResult>&&)> Callback)
{
	FPendingSearch PendingSearch;
	PendingSearch.QueryText = MoveTemp(QueryText);
	PendingSearch.QueryEmbedding = MoveTemp(QueryEmbedding);
	PendingSearch.K = K;
	PendingSearch.IDFilter = MoveTemp(IDFilter);
	PendingSearch.DistanceCutoff = DistanceCutoff;
	PendingSearch.Callback = MoveTemp(Callback);

	QueueData->PendingQueries.Enqueue(FPendingQuery(TInPlaceType<FPendingSearch>{}, MoveTemp(PendingSearch)));
	EnsureProcessingTask(UE::Tasks::ETaskPriority::Normal);
}

void FHybridSearchIndex::QuantizeAsync(TArray<float> Embedding,
	TFunction<void(TArray<uint8>&&, bool)> Callback)
{
	FPendingQuantize Pending;
	Pending.Embedding = MoveTemp(Embedding);
	Pending.Callback = MoveTemp(Callback);

	QueueData->PendingQueries.Enqueue(FPendingQuery(TInPlaceType<FPendingQuantize>{}, MoveTemp(Pending)));
	EnsureProcessingTask(UE::Tasks::ETaskPriority::Normal);
}

void FHybridSearchIndex::ContainsAsync(TArray<int64> IDs,
	TFunction<void(TSet<int64>&&)> Callback)
{
	FPendingContains Pending;
	Pending.IDs = MoveTemp(IDs);
	Pending.Callback = MoveTemp(Callback);

	QueueData->PendingQueries.Enqueue(FPendingQuery(TInPlaceType<FPendingContains>{}, MoveTemp(Pending)));
	EnsureProcessingTask(UE::Tasks::ETaskPriority::Normal);
}

void FHybridSearchIndex::ExtractEmbeddingsAsync(TArray<int64> CandidateIDs,
	TFunction<void(TArray<int64>&&, TArray<float>&&, int32)> Callback)
{
	FPendingExtractTraining Pending;
	Pending.CandidateIDs = MoveTemp(CandidateIDs);
	Pending.Callback = MoveTemp(Callback);

	QueueData->PendingQueries.Enqueue(FPendingQuery(TInPlaceType<FPendingExtractTraining>{}, MoveTemp(Pending)));
	EnsureProcessingTask(UE::Tasks::ETaskPriority::Normal);
}

void FHybridSearchIndex::IsFailedAsync(TArray<int64> IDs,
	TFunction<void(TSet<int64>&&)> Callback)
{
	FPendingIsFailed Pending;
	Pending.IDs = MoveTemp(IDs);
	Pending.Callback = MoveTemp(Callback);

	QueueData->PendingQueries.Enqueue(FPendingQuery(TInPlaceType<FPendingIsFailed>{}, MoveTemp(Pending)));
	EnsureProcessingTask(UE::Tasks::ETaskPriority::Normal);
}

void FHybridSearchIndex::IsRetryableFailedAsync(TArray<int64> IDs,
	TFunction<void(TSet<int64>&&)> Callback)
{
	FPendingIsRetryableFailed Pending;
	Pending.IDs = MoveTemp(IDs);
	Pending.Callback = MoveTemp(Callback);

	QueueData->PendingQueries.Enqueue(FPendingQuery(TInPlaceType<FPendingIsRetryableFailed>{}, MoveTemp(Pending)));
	EnsureProcessingTask(UE::Tasks::ETaskPriority::Normal);
}

void FHybridSearchIndex::DiscardPendingMutations()
{
	QueueData->bDiscardMutations.store(true, std::memory_order_release);
}

void FHybridSearchIndex::ProcessCommandBatch()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FHybridSearchIndex::ProcessCommandBatch);

	if (QueueData->bShutdownRequested.load(std::memory_order_acquire) || IsEngineExitRequested())
	{
		FPendingQuery DiscardedQuery;
		while (QueueData->PendingQueries.Dequeue(DiscardedQuery))
		{
		}
		FPendingMutation DiscardedMutation;
		while (QueueData->PendingMutations.Dequeue(DiscardedMutation))
		{
		}
		QueueData->bTaskInFlight.store(false, std::memory_order_release);
		return;
	}

	// Collected all async read event
	FGraphEventArray PendingIndexReadEvents;

	// --- High priority: drain and process all queued queries ---
	{
		FPendingQuery Query;
		while (QueueData->PendingQueries.Dequeue(Query))
		{
			if (FPendingSearch* Search = Query.TryGet<FPendingSearch>())
			{
				SearchDirect(
					MoveTemp(Search->QueryText),
					MoveTemp(Search->QueryEmbedding),
					Search->K,
					MoveTemp(Search->IDFilter),
					Search->DistanceCutoff,
					&PendingIndexReadEvents,
					[Callback = MoveTemp(Search->Callback)](TArray<FHybridSearchResult>&& Results) mutable
					{
						AsyncTask(ENamedThreads::GameThread,
							[Callback = MoveTemp(Callback), Results = MoveTemp(Results)]() mutable
							{
								Callback(MoveTemp(Results));
							});
					});
			}
			else if (FPendingQuantize* Quantize = Query.TryGet<FPendingQuantize>())
			{
				TArray<uint8> QuantizedCodes;
				bool bSuccess = false;

				if (VectorIndex && VectorIndex->SupportsQuantization() && VectorIndex->IsTrained())
				{
					auto* QuantizedIdx = static_cast<IQuantizedVectorIndex*>(VectorIndex.Get());
					QuantizedCodes = QuantizedIdx->Quantize(Quantize->Embedding);
					bSuccess = true;
				}

				Quantize->Callback(MoveTemp(QuantizedCodes), bSuccess);
			}
			else if (FPendingContains* ContainsReq = Query.TryGet<FPendingContains>())
			{
				TSet<int64> ContainedIDs;
				for (int64 ID : ContainsReq->IDs)
				{
					const bool bInVector = VectorIndex && VectorIndex->Contains(ID);
					if (!IsHybridEnabled())
					{
						if (bInVector) { ContainedIDs.Add(ID); }
					}
					else
					{
						if (bInVector && BM25Index->Contains(ID)) { ContainedIDs.Add(ID); }
					}
				}

				TFunction<void(TSet<int64>&&)> Callback = MoveTemp(ContainsReq->Callback);
				AsyncTask(ENamedThreads::GameThread,
					[Callback = MoveTemp(Callback), ContainedIDs = MoveTemp(ContainedIDs)]() mutable
					{
						Callback(MoveTemp(ContainedIDs));
					});
			}
			else if (FPendingExtractTraining* Extract = Query.TryGet<FPendingExtractTraining>())
			{
				TArray<int64> FoundIDs;
				TArray<float> FoundVectors;
				int32 Dim = VectorIndex ? VectorIndex->GetDimension() : 0;

				if (VectorIndex)
				{
					for (int64 ID : Extract->CandidateIDs)
					{
						if (!VectorIndex->Contains(ID)) { continue; }

						TArray<float> Embedding;
						if (VectorIndex->TryGetEmbedding(ID, Embedding))
						{
							FoundIDs.Add(ID);
							FoundVectors.Append(Embedding);
						}
					}
				}

				TFunction<void(TArray<int64>&&, TArray<float>&&, int32)> Callback = MoveTemp(Extract->Callback);
				AsyncTask(ENamedThreads::GameThread,
					[Callback = MoveTemp(Callback), FoundIDs = MoveTemp(FoundIDs), FoundVectors = MoveTemp(FoundVectors), Dim]() mutable
					{
						Callback(MoveTemp(FoundIDs), MoveTemp(FoundVectors), Dim);
					});
			}
			else if (FPendingIsFailed* IsFailedReq = Query.TryGet<FPendingIsFailed>())
			{
				// Returns the UNION of retryable and pre-processor failed IDs — used by skip-on-retry
				// paths that treat any failure as "don't attempt again during this session".
				TSet<int64> FailedIDs;
				for (int64 ID : IsFailedReq->IDs)
				{
					if (FailedAssets.Contains(ID) || PreProcessorFailedAssets.Contains(ID))
					{
						FailedIDs.Add(ID);
					}
				}

				TFunction<void(TSet<int64>&&)> Callback = MoveTemp(IsFailedReq->Callback);
				AsyncTask(ENamedThreads::GameThread,
					[Callback = MoveTemp(Callback), FailedIDs = MoveTemp(FailedIDs)]() mutable
					{
						Callback(MoveTemp(FailedIDs));
					});
			}
			else if (FPendingIsRetryableFailed* IsRetryableReq = Query.TryGet<FPendingIsRetryableFailed>())
			{
				// Returns only retryable failures (excludes pre-processor). Used by the "Retry failed"
				// button so permanent failures never loop into a retry that will re-fail.
				TSet<int64> FailedIDs;
				for (int64 ID : IsRetryableReq->IDs)
				{
					if (FailedAssets.Contains(ID))
					{
						FailedIDs.Add(ID);
					}
				}

				TFunction<void(TSet<int64>&&)> Callback = MoveTemp(IsRetryableReq->Callback);
				AsyncTask(ENamedThreads::GameThread,
					[Callback = MoveTemp(Callback), FailedIDs = MoveTemp(FailedIDs)]() mutable
					{
						Callback(MoveTemp(FailedIDs));
					});
			}
		}
	}

	// --- Normal priority: drain and batch process mutations ---

	if (PendingIndexReadEvents.Num() > 0)
	{
		// Make sure all read event are finish before executing any queued modifications
		FTaskGraphInterface::Get().WaitUntilTasksComplete(PendingIndexReadEvents, ENamedThreads::AnyBackgroundHiPriTask);
		PendingIndexReadEvents.Reset();
	}

	// If a discard was requested (e.g. CancelIndexing), drain and skip
	if (QueueData->bDiscardMutations.exchange(false, std::memory_order_acquire))
	{
		FPendingMutation Discarded;
		while (QueueData->PendingMutations.Dequeue(Discarded)) {}
	}
	else
	{
		TArray<FPendingMutation, TInlineAllocator<64>> Mutations;
		{
			FPendingMutation Mutation;
			while (QueueData->PendingMutations.Dequeue(Mutation))
			{
				Mutations.Add(MoveTemp(Mutation));
			}
		}

		if (Mutations.Num() > 0)
		{
			const uint32 CurrentIndexId = GetIndexId();
			bool bChanged = false;

			// Partition mutations: order-dependent commands execute immediately in order,
			// while Add/AddQuantized/Remove/MarkFailed are collected for batching.
			TArray<int64> BatchRemoveIDs;
			TArray<FAssetData> BatchAddAssets;
			TArray<float> BatchAddEmbeddings;
			TArray<FString> BatchAddCaptions;
			TArray<TArray<FString>> BatchAddKeywords;
			TArray<FAssetData> BatchAddQuantizedAssets;
			TArray<TArray<uint8>> BatchAddQuantizedCodes;
			TArray<FString> BatchAddQuantizedCaptions;
			TArray<TArray<FString>> BatchAddQuantizedKeywords;
			// Pair each batched MarkFailed with its reason so the consumer routes to the right set.
			TArray<TPair<int64, EAssetIndexFailureReason>> BatchMarkFailed;
			TArray<int64> BatchClearFailedIDs;
			TArray<TSet<int64>> BatchPurgePreProcValidSets;
			TArray<FMutationSave> DeferredSaves;

			auto ResetBatches = [&]()
			{
				BatchRemoveIDs.Reset();
				BatchAddAssets.Reset();
				BatchAddEmbeddings.Reset();
				BatchAddCaptions.Reset();
				BatchAddKeywords.Reset();
				BatchAddQuantizedAssets.Reset();
				BatchAddQuantizedCodes.Reset();
				BatchAddQuantizedCaptions.Reset();
				BatchAddQuantizedKeywords.Reset();
				BatchMarkFailed.Reset();
				BatchClearFailedIDs.Reset();
				BatchPurgePreProcValidSets.Reset();
			};

			for (FPendingMutation& MutationCommand : Mutations)
			{
				if (FMutationSetVectorIndex* SetVectorIdx = MutationCommand.TryGet<FMutationSetVectorIndex>())
				{
					// Discard any batched mutations  - they targeted the old index
					ResetBatches();
					SetVectorIndexDirect(MoveTemp(SetVectorIdx->NewVectorIndex));
					bChanged = true;
				}
				else if (MutationCommand.IsType<FMutationClearBM25>())
				{
					GetBM25Index().Clear();
					bChanged = true;
				}
				else if (MutationCommand.IsType<FMutationResetBM25AndFailed>())
				{
					// Used only by force-rebuild / index-type switch / retrain — all "start over" flows
					// where clearing the persistent pre-processor bucket is correct.
					ResetBatches();
					GetBM25Index().Clear();
					FailedAssets.Empty();
					PreProcessorFailedAssets.Empty();
					bChanged = true;
				}
				else if (FMutationSave* SaveCmd = MutationCommand.TryGet<FMutationSave>())
				{
					// Defer saves to run after all other mutations in this batch
					DeferredSaves.Add(MoveTemp(*SaveCmd));
				}
				else if (FMutationRemoveAsset* RemoveAsset = MutationCommand.TryGet<FMutationRemoveAsset>())
				{
					BatchRemoveIDs.Add(GetAssetIndexID(RemoveAsset->Asset));
				}
				else if (FMutationRemovePath* RemovePath = MutationCommand.TryGet<FMutationRemovePath>())
				{
					BatchRemoveIDs.Add(GetAssetIndexID(RemovePath->AssetPath));
				}
				else if (FMutationRemoveId* RemoveId = MutationCommand.TryGet<FMutationRemoveId>())
				{
					BatchRemoveIDs.Add(RemoveId->AssetId);
				}
				else if (FMutationAdd* AddCmd = MutationCommand.TryGet<FMutationAdd>())
				{
					if (AddCmd->IndexId == CurrentIndexId)
					{
						BatchRemoveIDs.Add(GetAssetIndexID(AddCmd->Asset));
						BatchAddAssets.Add(MoveTemp(AddCmd->Asset));
						BatchAddEmbeddings.Append(MoveTemp(AddCmd->Embedding));
						BatchAddCaptions.Add(MoveTemp(AddCmd->Caption));
						BatchAddKeywords.Add(MoveTemp(AddCmd->Keywords));
							}
				}
				else if (FMutationAddQuantized* AddQuantized = MutationCommand.TryGet<FMutationAddQuantized>())
				{
					if (AddQuantized->IndexId == CurrentIndexId)
					{
						BatchRemoveIDs.Add(GetAssetIndexID(AddQuantized->Asset));
						BatchAddQuantizedAssets.Add(MoveTemp(AddQuantized->Asset));
						BatchAddQuantizedCodes.Add(MoveTemp(AddQuantized->QuantizedCodes));
						BatchAddQuantizedCaptions.Add(MoveTemp(AddQuantized->Caption));
						BatchAddQuantizedKeywords.Add(MoveTemp(AddQuantized->Keywords));
							}
				}
				else if (FMutationMarkFailed* MarkFailed = MutationCommand.TryGet<FMutationMarkFailed>())
				{
					if (MarkFailed->IndexId == CurrentIndexId)
					{
						BatchMarkFailed.Emplace(MarkFailed->AssetId, MarkFailed->Reason);
							}
				}
				else if (FMutationClearFailed* ClearFailed = MutationCommand.TryGet<FMutationClearFailed>())
				{
					BatchClearFailedIDs.Add(ClearFailed->AssetId);
				}
				else if (FMutationPurgePreProcessorFailedNotIn* Purge = MutationCommand.TryGet<FMutationPurgePreProcessorFailedNotIn>())
				{
					BatchPurgePreProcValidSets.Add(MoveTemp(Purge->ValidIDs));
				}
			}

			// Engine teardown may have begun during the accumulation loop above; bail
			// before the batched executes (AddDirect / IsHybridEnabled / SerializeIndexForSave
			// all read USemanticSearchSettings::Get, unsafe after UObject teardown).
			if (QueueData->bShutdownRequested.load(std::memory_order_acquire) || IsEngineExitRequested())
			{
				QueueData->bTaskInFlight.store(false, std::memory_order_release);
				return;
			}

			// Execute batched operations
			if (BatchRemoveIDs.Num() > 0)
			{
				RemoveDirect(BatchRemoveIDs);
				bChanged = true;
			}

			if (BatchAddAssets.Num() > 0)
			{
				AddDirect(BatchAddAssets, BatchAddEmbeddings, BatchAddCaptions, BatchAddKeywords);
				bChanged = true;
			}

			if (BatchAddQuantizedAssets.Num() > 0)
			{
				AddQuantizedDirect(BatchAddQuantizedAssets, BatchAddQuantizedCodes,
					BatchAddQuantizedCaptions, BatchAddQuantizedKeywords);
				bChanged = true;
			}

			if (BatchMarkFailed.Num() > 0)
			{
				for (const TPair<int64, EAssetIndexFailureReason>& Pair : BatchMarkFailed)
				{
					MarkFailedDirect(Pair.Key, Pair.Value);
				}
				bChanged = true;
			}

			if (BatchClearFailedIDs.Num() > 0)
			{
				for (int64 ID : BatchClearFailedIDs)
				{
					ClearFailedStateDirect(ID);
				}
				bChanged = true;
			}

			if (BatchPurgePreProcValidSets.Num() > 0)
			{
				for (const TSet<int64>& ValidIDs : BatchPurgePreProcValidSets)
				{
					PurgePreProcessorFailedNotInDirect(ValidIDs);
				}
				bChanged = true;
			}

			// Saves run last so they capture the fully-mutated index state.
			if (DeferredSaves.Num() > 0)
			{
				FMutationSave& SaveCmd = DeferredSaves.Last();

				TArray64<uint8> IndexBytes = SerializeIndexForSave(SaveCmd.PackageHashes);
				FString IndexPath = MoveTemp(SaveCmd.FilePath);
				QueueData->SavePipe.Launch(TEXT("HybridSearchSaveIndex"),
					[Path = MoveTemp(IndexPath), Bytes = MoveTemp(IndexBytes)]() mutable
					{
						WriteBytesToFile(Path, MoveTemp(Bytes));
					},
					UE::Tasks::ETaskPriority::BackgroundNormal);

				if (!SaveCmd.PreProcessorFailedPath.IsEmpty())
				{
					TArray64<uint8> FailedBytes = SerializePreProcessorFailedForSave();
					FString FailedPath = MoveTemp(SaveCmd.PreProcessorFailedPath);
					QueueData->SavePipe.Launch(TEXT("HybridSearchSaveFailed"),
						[Path = MoveTemp(FailedPath), Bytes = MoveTemp(FailedBytes)]() mutable
						{
							WriteBytesToFile(Path, MoveTemp(Bytes));
						},
						UE::Tasks::ETaskPriority::BackgroundNormal);
				}
			}

			// Snapshot stats so game-thread readers see a consistent, lock-free view.
			if (bChanged)
			{
				UpdateCachedStats();
			}
		}
	}

	// Consume the priority boost flag before releasing
	const bool bBoost = QueueData->bPriorityBoostRequested.exchange(false, std::memory_order_acquire);

	// Release the in-flight flag before checking for new work
	QueueData->bTaskInFlight.store(false, std::memory_order_release);

	// If new work arrived while we were processing, or a priority boost was requested, re-launch
	if (!QueueData->PendingMutations.IsEmpty() || !QueueData->PendingQueries.IsEmpty() || bBoost)
	{
		EnsureProcessingTask(bBoost ? UE::Tasks::ETaskPriority::Normal : UE::Tasks::ETaskPriority::BackgroundNormal);
	}
}

// --- End command queue methods ---

void FHybridSearchIndex::EnsureInitialized(int32 EmbeddingDimension)
{
	if (bInitialized.load(std::memory_order_relaxed) || EmbeddingDimension <= 0)
	{
		return;
	}

	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();

	ESemanticSearchIndexType Type = Settings ? Settings->IndexType : ESemanticSearchIndexType::Flat;
	const FString CodebookPath = GetCodebookPath(Type);

	if (!CodebookPath.IsEmpty())
	{
		TArray<uint8> CodebookData;
		if (FPaths::FileExists(CodebookPath) &&
			FFileHelper::LoadFileToArray(CodebookData, *CodebookPath))
		{
			VectorIndex = DeserializeCodebook(Type, CodebookData, EmbeddingDimension);
			if (VectorIndex)
			{
				UE_LOGF(LogSemanticSearch, Log, "Loaded codebook from %ls", *CodebookPath);
				bInitialized.store(true, std::memory_order_release);
				return;
			}
		}
	}

	UE_LOGF(LogSemanticSearch, Display, "No codebook found, creating flat index");
	Type = ESemanticSearchIndexType::Flat;
	VectorIndex = CreateVectorIndex(Type, EmbeddingDimension, Settings);
	USemanticSearchSettings* MutableSettings = GetMutableDefault<USemanticSearchSettings>();
	MutableSettings->IndexType = Type;
	MutableSettings->SaveConfig();
	bInitialized.store(true, std::memory_order_release);
}

TSharedPtr<IVectorIndex> FHybridSearchIndex::GetVectorIndex() const
{
	return VectorIndex;
}

void FHybridSearchIndex::SetVectorIndexDirect(TSharedPtr<IVectorIndex> NewIndex)
{
	VectorIndex = MoveTemp(NewIndex);
	if (VectorIndex)
	{
		bInitialized.store(true, std::memory_order_release);
	}
	// Note: IndexId is NOT changed here. The caller (CancelIndexing) is responsible
	// for calling InvalidateIndexId() before enqueuing SetVectorIndex, so the game
	// thread can capture the correct IndexId for subsequent TriggerIndexing calls.
}

uint32 FHybridSearchIndex::GetIndexId() const
{
	return IndexId.load(std::memory_order_acquire);
}

void FHybridSearchIndex::InvalidateIndexId()
{
	IndexId.store(FGuid::NewGuid().A, std::memory_order_release);
}

bool FHybridSearchIndex::IsHybridEnabled() const
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	return Settings && Settings->bEnableHybridSearch;
}

FBM25Index& FHybridSearchIndex::GetBM25Index()
{
	return *BM25Index;
}

void FHybridSearchIndex::ResetBM25AndFailedState()
{
	QueueData->PendingMutations.Enqueue(
		FPendingMutation(TInPlaceType<FMutationResetBM25AndFailed>{}));
	EnsureProcessingTask();
}

void FHybridSearchIndex::UpdateCachedStats()
{
	TSharedRef<FCachedIndexStats, ESPMode::ThreadSafe> NewStats = MakeShared<FCachedIndexStats, ESPMode::ThreadSafe>();
	NewStats->bIsInitialized = bInitialized.load(std::memory_order_relaxed);
	if (VectorIndex)
	{
		NewStats->VectorCount = static_cast<int32>(VectorIndex->GetCount());
		NewStats->Dimension = VectorIndex->GetDimension();
		NewStats->bIsTrained = VectorIndex->IsTrained();
		NewStats->EstimatedVectorMemoryBytes = VectorIndex->EstimateMemoryBytes();
		NewStats->bSupportsQuantization = VectorIndex->SupportsQuantization();
		if (VectorIndex->SupportsQuantization() && VectorIndex->IsTrained())
		{
			NewStats->CodebookHash = static_cast<IQuantizedVectorIndex*>(VectorIndex.Get())->GetCodebookHash();
		}
	}
	NewStats->BM25Count = static_cast<int32>(BM25Index->GetCount());
	NewStats->EstimatedBM25MemoryBytes = BM25Index->EstimateMemoryBytes();
	NewStats->FailedCount = FailedAssets.Num();
	NewStats->PreProcessorFailedCount = PreProcessorFailedAssets.Num();

	{
		FScopeLock Lock(&QueueData->CachedStatsLock);
		QueueData->CachedStats = NewStats;
	}
}

FSemanticSearchIndexStats FHybridSearchIndex::GetCachedIndexStats() const
{
	TSharedPtr<const FCachedIndexStats, ESPMode::ThreadSafe> Cached;
	{
		FScopeLock Lock(&QueueData->CachedStatsLock);
		Cached = QueueData->CachedStats;
	}
	FSemanticSearchIndexStats Stats;
	if (Cached)
	{
		Stats.VectorCount = Cached->VectorCount;
		Stats.BM25Count = Cached->BM25Count;
		Stats.Dimension = Cached->Dimension;
		Stats.EstimatedVectorMemoryBytes = Cached->EstimatedVectorMemoryBytes;
		Stats.EstimatedBM25MemoryBytes = Cached->EstimatedBM25MemoryBytes;
		Stats.bIsTrained = Cached->bIsTrained;
		Stats.bSupportsQuantization = Cached->bSupportsQuantization;
		Stats.bIsInitialized = Cached->bIsInitialized;
		Stats.FailedCount = Cached->FailedCount;
		Stats.PreProcessorFailedCount = Cached->PreProcessorFailedCount;
	}
	return Stats;
}

FIoHash FHybridSearchIndex::GetCachedCodebookHash() const
{
	TSharedPtr<const FCachedIndexStats, ESPMode::ThreadSafe> Cached;
	{
		FScopeLock Lock(&QueueData->CachedStatsLock);
		Cached = QueueData->CachedStats;
	}
	return Cached ? Cached->CodebookHash : FIoHash();
}

void FHybridSearchIndex::MarkFailedDirect(int64 AssetID, EAssetIndexFailureReason Reason)
{
	if (IsPreProcessorFailure(Reason))
	{
		PreProcessorFailedAssets.Add(AssetID);
		// If this asset was previously in the retryable bucket, move it over.
		FailedAssets.Remove(AssetID);
	}
	else
	{
		FailedAssets.Add(AssetID);
	}
}

void FHybridSearchIndex::ClearFailedStateDirect(int64 AssetID)
{
	FailedAssets.Remove(AssetID);
	PreProcessorFailedAssets.Remove(AssetID);
}

void FHybridSearchIndex::PurgePreProcessorFailedNotInDirect(const TSet<int64>& ValidIDs)
{
	for (auto It = PreProcessorFailedAssets.CreateIterator(); It; ++It)
	{
		if (!ValidIDs.Contains(*It))
		{
			It.RemoveCurrent();
		}
	}
}

int64 GetAssetIndexID(FStringView AssetPath)
{
	FBlake3 Hasher;
	Hasher.Update(AssetPath.GetData(), AssetPath.Len() * sizeof(TCHAR));
	FBlake3Hash Hash = Hasher.Finalize();
	int64 Result;
	FMemory::Memcpy(&Result, Hash.GetBytes(), sizeof(int64));
	// Ensure non-negative (FAISS uses -1 as sentinel)
	return Result & 0x7FFFFFFFFFFFFFFF;
}

int64 GetAssetIndexID(const FAssetData& Asset)
{
	return GetAssetIndexID(Asset.GetObjectPathString());
}

void FHybridSearchIndex::AddDirect(
	const FAssetData& Asset,
	TConstArrayView<float> Embedding,
	FStringView Caption,
	TConstArrayView<FString> Keywords)
{
	const int64 ID = GetAssetIndexID(Asset);

	// Lazy-init vector index from embedding dimension
	if (Embedding.Num() > 0)
	{
		EnsureInitialized(Embedding.Num());
	}

	// Add to vector index
	if (VectorIndex && Embedding.Num() > 0)
	{
		VectorIndex->Add(ID, Embedding);
	}

	// Add to BM25 index
	if (IsHybridEnabled())
	{
		BM25Index->Add(ID, Asset.GetObjectPathString(), Caption, Keywords);
	}

	// Clear any previous failure now that this asset indexed successfully (both buckets).
	FailedAssets.Remove(ID);
	PreProcessorFailedAssets.Remove(ID);
}

void FHybridSearchIndex::AddDirect(
	TConstArrayView<FAssetData> Assets,
	TConstArrayView<float> AllEmbeddings,
	TConstArrayView<FString> Captions,
	const TArray<TArray<FString>>& AllKeywords)
{
	if (Assets.Num() == 0)
	{
		return;
	}

	// Compute IDs
	TArray<int64> IDs;
	IDs.SetNum(Assets.Num());
	for (int32 i = 0; i < Assets.Num(); ++i)
	{
		IDs[i] = GetAssetIndexID(Assets[i]);
	}

	// Lazy-init from first embedding dimension
	const int32 Dim = Assets.Num() > 0 && AllEmbeddings.Num() > 0
		? AllEmbeddings.Num() / Assets.Num() : 0;
	if (Dim > 0)
	{
		EnsureInitialized(Dim);
	}

	// Batch add to vector index
	if (VectorIndex && AllEmbeddings.Num() > 0)
	{
		VectorIndex->Add(IDs, AllEmbeddings);
	}

	// Batch add to BM25 index
	if (IsHybridEnabled())
	{
		TArray<FString> Paths;
		Paths.SetNum(Assets.Num());
		for (int32 i = 0; i < Assets.Num(); ++i)
		{
			Paths[i] = Assets[i].GetObjectPathString();
		}
		BM25Index->Add(IDs, Paths, Captions, AllKeywords);
	}

	// Clear any previous failures (both buckets).
	for (int64 ID : IDs)
	{
		FailedAssets.Remove(ID);
		PreProcessorFailedAssets.Remove(ID);
	}
}

void FHybridSearchIndex::AddQuantizedDirect(
	const FAssetData& Asset,
	TConstArrayView<uint8> QuantizedCodes,
	FStringView Caption,
	TConstArrayView<FString> Keywords)
{
	const int64 ID = GetAssetIndexID(Asset);

	check(VectorIndex && VectorIndex->SupportsQuantization() && VectorIndex->IsTrained());

	static_cast<IQuantizedVectorIndex*>(VectorIndex.Get())->AddQuantized(ID, QuantizedCodes);

	if (IsHybridEnabled())
	{
		BM25Index->Add(ID, Asset.GetObjectPathString(), Caption, Keywords);
	}

	FailedAssets.Remove(ID);
	PreProcessorFailedAssets.Remove(ID);
}

void FHybridSearchIndex::AddQuantizedDirect(
	TConstArrayView<FAssetData> Assets,
	const TArray<TArray<uint8>>& AllQuantizedCodes,
	TConstArrayView<FString> Captions,
	const TArray<TArray<FString>>& AllKeywords)
{
	if (Assets.Num() == 0)
	{
		return;
	}

	check(VectorIndex && VectorIndex->SupportsQuantization() && VectorIndex->IsTrained());

	// Compute IDs and concatenate codes
	TArray<int64> IDs;
	IDs.SetNum(Assets.Num());
	TArray<uint8> ConcatenatedCodes;
	for (int32 i = 0; i < Assets.Num(); ++i)
	{
		IDs[i] = GetAssetIndexID(Assets[i]);
		ConcatenatedCodes.Append(AllQuantizedCodes[i]);
	}

	// Batch add to quantized vector index
	static_cast<IQuantizedVectorIndex*>(VectorIndex.Get())->AddQuantized(IDs, ConcatenatedCodes);

	// Batch add to BM25 index
	if (IsHybridEnabled())
	{
		TArray<FString> Paths;
		Paths.SetNum(Assets.Num());
		for (int32 i = 0; i < Assets.Num(); ++i)
		{
			Paths[i] = Assets[i].GetObjectPathString();
		}
		BM25Index->Add(IDs, Paths, Captions, AllKeywords);
	}

	// Clear any previous failures (both buckets).
	for (int64 ID : IDs)
	{
		FailedAssets.Remove(ID);
		PreProcessorFailedAssets.Remove(ID);
	}
}

void FHybridSearchIndex::RemoveDirect(const FAssetData& Asset)
{
	RemoveDirect(Asset.GetObjectPathString());
}

void FHybridSearchIndex::RemoveDirect(FStringView AssetPath)
{
	RemoveDirect(GetAssetIndexID(AssetPath));
}

void FHybridSearchIndex::RemoveDirect(int64 AssetID)
{
	if (VectorIndex)
	{
		VectorIndex->Remove(AssetID);
	}
	BM25Index->Remove(AssetID);
	FailedAssets.Remove(AssetID);
	PreProcessorFailedAssets.Remove(AssetID);
}

void FHybridSearchIndex::RemoveDirect(TConstArrayView<int64> IDs)
{
	if (IDs.Num() == 0)
	{
		return;
	}

	if (VectorIndex)
	{
		VectorIndex->Remove(IDs);
	}
	BM25Index->Remove(IDs);
	for (int64 ID : IDs)
	{
		FailedAssets.Remove(ID);
		PreProcessorFailedAssets.Remove(ID);
	}
}

bool FHybridSearchIndex::Contains(const FAssetData& Asset) const
{
	const int64 ID = GetAssetIndexID(Asset);
	const bool bInVector = VectorIndex && VectorIndex->Contains(ID);

	if (!IsHybridEnabled())
	{
		return bInVector;
	}

	return bInVector && BM25Index->Contains(ID);
}

void FHybridSearchIndex::SearchDirect(
	FString QueryText,
	TArray<float> QueryEmbedding,
	int32 K,
	TArray<int64> IDFilter,
	float DistanceCutoff,
	FGraphEventArray* OutIndexReadEvents,
	TFunction<void(TArray<FHybridSearchResult>&&)> Continuation) const
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	const int32 RRFk = Settings ? Settings->RRFConstant : 60;
	const int32 Oversample = Settings ? Settings->RRFOversample : 3;

	// Fetch more candidates than K from each source for better fusion.
	const int32 FetchK = K * Oversample;

	const bool bWantsVector = VectorIndex && QueryEmbedding.Num() > 0;
	const bool bWantsText = IsHybridEnabled() && QueryText.Len() > 0;

	if (!bWantsVector && !bWantsText)
	{
		Continuation(TArray<FHybridSearchResult>{});
		return;
	}

	struct FSearchState
	{
		TArray<FSearchResult> VectorResults;
		TArray<FBM25Result> TextResults;
		// The search try to reuse the larger buffer when doing the sort of the fusion ranking to reduce the physical page miss
		TArray<uint32> VectorScratch;
		TArray<uint32> TextScratch;
		TFunction<void(TArray<FHybridSearchResult>&&)> Continuation;
		int32 K = 0;
		int32 RRFk = 0;
		int32 NumDispatch = 0;

		// Populated by the first search that finished
		bool bFirstWasVector = false;
		int32 N1 = 0;

		TArray<FHybridSearchResult> Results;
		FHashTable IDTable;

		// Pre-created in the dispatch code below so it exists before either
		// continuation runs. First arriver dispatches SeedFusionTable task that calls
		// DispatchSubsequents() at end. Second arriver passes this in prereqs.
		FGraphEventRef FusionTableSeededEvent;

		std::atomic<int32> Pending{0};
	};
	using FSharedState = TSharedPtr<FSearchState, ESPMode::ThreadSafe>;

	// Single-source path: only one of vector/text was dispatched. No fusion.
	auto EmitSingleSourceResults = [](FSharedState State)
	{
		const int32 LocalK = State->K;
		const int32 LocalRRFk = State->RRFk;
		TArray<FSearchResult>& VectorResults = State->VectorResults;
		TArray<FBM25Result>& TextResults = State->TextResults;

		if (VectorResults.Num() == 0 && TextResults.Num() == 0)
		{
			State->Continuation(TArray<FHybridSearchResult>{});
			return;
		}

		if (TextResults.Num() == 0)
		{
			TArray<FHybridSearchResult> Results;
			Results.Reserve(FMath::Min(LocalK, VectorResults.Num()));
			for (int32 i = 0; i < VectorResults.Num() && i < LocalK; ++i)
			{
				FHybridSearchResult R;
				R.ID = VectorResults[i].ID;
				R.VectorDistance = VectorResults[i].Distance;
				R.RRFScore = 1.0f / (LocalRRFk + i + 1);
				Results.Add(R);
			}
			State->Continuation(MoveTemp(Results));
			return;
		}

		// VectorResults empty, TextResults non-empty.
		TArray<FHybridSearchResult> Results;
		Results.Reserve(FMath::Min(LocalK, TextResults.Num()));
		for (int32 i = 0; i < TextResults.Num() && i < LocalK; ++i)
		{
			FHybridSearchResult R;
			R.ID = TextResults[i].ID;
			R.BM25Score = TextResults[i].Score;
			R.RRFScore = 1.0f / (LocalRRFk + i + 1);
			Results.Add(R);
		}
		State->Continuation(MoveTemp(Results));
	};

	// SeedFusionTable: insert the FIRST search list into Results+IDTable.
	auto SeedFusionTable = [](FSharedState State)
	{
		const int32 N1 = State->N1;
		const int32 LocalRRFk = State->RRFk;
		const bool bFirstIsVector = State->bFirstWasVector;

		// IDTable sized for exactly N1 writers
		const uint32 IDTableHashSize = FMath::Max(32u, FMath::RoundUpToPowerOfTwo(static_cast<uint32>(FMath::Max(N1, 1))));
		State->IDTable.Clear(IDTableHashSize, static_cast<uint32>(N1));

		// Reserve up to N1 + N2 in MergeAndRank (we don't know N2 here yet).
		// For now size to N1; MergeAndRank will grow once second arriver lands.
		State->Results.SetNumUninitialized(N1);

		if (N1 == 0)
		{
			// Nothing to do — MergeAndRank will rebuild Results from second list.
			State->FusionTableSeededEvent->DispatchSubsequents();
			return;
		}

		constexpr int32 FusionMinBatchSize = 8192;

		TArray<FHybridSearchResult>& Results = State->Results;
		FHashTable& IDTable = State->IDTable;

		if (bFirstIsVector)
		{
			TArray<FSearchResult>& VectorResults = State->VectorResults;
			ParallelFor(TEXT("FHybridSearchIndex::SearchDirect Fusion: Seed (vector)"),
				N1, FusionMinBatchSize,
				[&Results, &IDTable, &VectorResults, LocalRRFk](int32 i)
				{
					const int64 ID = VectorResults[i].ID;
					FHybridSearchResult& R = Results[i];
					R.ID = ID;
					R.VectorDistance = VectorResults[i].Distance;
					R.BM25Score = -1.0f;
					R.RRFScore = 1.0f / (LocalRRFk + i + 1);
					IDTable.Add_Concurrent(::GetTypeHash(ID), static_cast<uint32>(i));
				});
		}
		else
		{
			TArray<FBM25Result>& TextResults = State->TextResults;
			ParallelFor(TEXT("FHybridSearchIndex::SearchDirect Fusion: Seed (text)"),
				N1, FusionMinBatchSize,
				[&Results, &IDTable, &TextResults, LocalRRFk](int32 i)
				{
					const int64 ID = TextResults[i].ID;
					FHybridSearchResult& R = Results[i];
					R.ID = ID;
					R.VectorDistance = -1.0f;
					R.BM25Score = TextResults[i].Score;
					R.RRFScore = 1.0f / (LocalRRFk + i + 1);
					IDTable.Add_Concurrent(::GetTypeHash(ID), static_cast<uint32>(i));
				});
		}

		State->FusionTableSeededEvent->DispatchSubsequents();
	};

	// MergeAndRank: Start when the second search is finished and the SeedFusionTable has been done
	auto MergeAndRank = [](FSharedState State)
	{
		const int32 LocalK = State->K;
		const int32 LocalRRFk = State->RRFk;
		const int32 N1 = State->N1;
		const bool bSecondIsVector = !State->bFirstWasVector;
		const int32 N2 = bSecondIsVector ? State->VectorResults.Num() : State->TextResults.Num();

		// Edge case: second arriver was empty
		if (N2 == 0)
		{
			TArray<FHybridSearchResult>& Results = State->Results;
			if (Results.Num() > LocalK)
			{
				Results.SetNum(LocalK);
			}
			State->Continuation(MoveTemp(Results));
			return;
		}

		
		TArray<FHybridSearchResult>& Results = State->Results;
		FHashTable& IDTable = State->IDTable;

		TArray<int32> MissL2Idx;
		MissL2Idx.SetNumUninitialized(N2);
		std::atomic<int32> MissCount{0};

		constexpr int32 FusionMinBatchSize = 8192;

		// Merge result that are present for both search
		if (bSecondIsVector)
		{
			TArray<FSearchResult>& VectorResults = State->VectorResults;
			ParallelFor(TEXT("FHybridSearchIndex::SearchDirect Fusion: phase 2 (vector find)"),
				N2, FusionMinBatchSize,
				[&Results, &IDTable, &VectorResults, &MissL2Idx, &MissCount, LocalRRFk](int32 i)
				{
					const int64 ID = VectorResults[i].ID;
					const uint32 Key = ::GetTypeHash(ID);
					int32 HitIdx = -1;
					for (uint32 j = IDTable.First(Key); IDTable.IsValid(j); j = IDTable.Next(j))
					{
						if (Results[j].ID == ID)
						{
							HitIdx = static_cast<int32>(j);
							break;
						}
					}
					if (HitIdx >= 0)
					{
						FHybridSearchResult& R = Results[HitIdx];
						R.VectorDistance = VectorResults[i].Distance;
						R.RRFScore += 1.0f / (LocalRRFk + i + 1);
					}
					else
					{
						const int32 Slot = MissCount.fetch_add(1, std::memory_order_relaxed);
						MissL2Idx[Slot] = i;
					}
				});
		}
		else
		{
			TArray<FBM25Result>& TextResults = State->TextResults;
			ParallelFor(TEXT("FHybridSearchIndex::SearchDirect Fusion: phase 2 (text find)"),
				N2, FusionMinBatchSize,
				[&Results, &IDTable, &TextResults, &MissL2Idx, &MissCount, LocalRRFk](int32 i)
				{
					const int64 ID = TextResults[i].ID;
					const uint32 Key = ::GetTypeHash(ID);
					int32 HitIdx = -1;
					for (uint32 j = IDTable.First(Key); IDTable.IsValid(j); j = IDTable.Next(j))
					{
						if (Results[j].ID == ID)
						{
							HitIdx = static_cast<int32>(j);
							break;
						}
					}
					if (HitIdx >= 0)
					{
						FHybridSearchResult& R = Results[HitIdx];
						R.BM25Score = TextResults[i].Score;
						R.RRFScore += 1.0f / (LocalRRFk + i + 1);
					}
					else
					{
						const int32 Slot = MissCount.fetch_add(1, std::memory_order_relaxed);
						MissL2Idx[Slot] = i;
					}
				});
		}



		const int32 ActualMissCount = MissCount.load(std::memory_order_relaxed);
		Results.SetNumUninitialized(N1 + ActualMissCount);

		// Write the search result that are unique to the second search
		if (bSecondIsVector)
		{
			TArray<FSearchResult>& VectorResults = State->VectorResults;
			ParallelFor(TEXT("FHybridSearchIndex::SearchDirect Fusion: phase 3 (vector miss)"),
				ActualMissCount, FusionMinBatchSize,
				[&Results, &VectorResults, &MissL2Idx, N1, LocalRRFk](int32 m)
				{
					const int32 i = MissL2Idx[m];
					FHybridSearchResult& R = Results[N1 + m];
					R.ID = VectorResults[i].ID;
					R.VectorDistance = VectorResults[i].Distance;
					R.BM25Score = -1.0f;
					R.RRFScore = 1.0f / (LocalRRFk + i + 1);
				});
		}
		else
		{
			TArray<FBM25Result>& TextResults = State->TextResults;
			ParallelFor(TEXT("FHybridSearchIndex::SearchDirect Fusion: phase 3 (text miss)"),
				ActualMissCount, FusionMinBatchSize,
				[&Results, &TextResults, &MissL2Idx, N1, LocalRRFk](int32 m)
				{
					const int32 i = MissL2Idx[m];
					FHybridSearchResult& R = Results[N1 + m];
					R.ID = TextResults[i].ID;
					R.VectorDistance = -1.0f;
					R.BM25Score = TextResults[i].Score;
					R.RRFScore = 1.0f / (LocalRRFk + i + 1);
				});
		}

		// Sort descending by RRF score. Reuse the biggest buffer
		TArray<uint32>& FusionScratch = (State->VectorScratch.Num() >= State->TextScratch.Num())
			? State->VectorScratch
			: State->TextScratch;
		Private::RadixSort(Results, FusionScratch, [](const FHybridSearchResult& R) { return R.RRFScore; }, /*bDescending*/ true);

		if (Results.Num() > LocalK)
		{
			Results.SetNum(LocalK);
		}

		State->Continuation(MoveTemp(Results));
	};

	FSharedState State = MakeShared<FSearchState, ESPMode::ThreadSafe>();
	State->Continuation = MoveTemp(Continuation);
	State->K = K;
	State->RRFk = RRFk;
	State->NumDispatch = (bWantsVector ? 1 : 0) + (bWantsText ? 1 : 0);

	// Pre-create the FusionTableSeeded event ONLY for the two-source case.
	if (State->NumDispatch >= 2)
	{
		State->FusionTableSeededEvent = FGraphEvent::CreateGraphEvent();
	}

	State->Pending.store(State->NumDispatch, std::memory_order_release);
	const TSharedRef<const TArray<int64>> SharedIDFilter = MakeShared<TArray<int64>>(MoveTemp(IDFilter));

	// Dispatch helper for an arriver: branches on NumDispatch / PrevPending.
	auto OnArrival = [EmitSingleSourceResults, SeedFusionTable, MergeAndRank](FSharedState State, bool bIsVector)
	{
		const int32 PrevPending = State->Pending.fetch_sub(1, std::memory_order_acq_rel);

		if (State->NumDispatch == 1)
		{
			// Only one source dispatched — single-source fast path, no fusion.
			EmitSingleSourceResults(State);
			return;
		}

		// NumDispatch == 2.
		if (PrevPending == 2)
		{
			State->bFirstWasVector = bIsVector;
			State->N1 = bIsVector ? State->VectorResults.Num() : State->TextResults.Num();

			Private::RunOrDispatchIndexWorker(
				[State, SeedFusionTable]() { SeedFusionTable(State); });
		}
		else
		{
			FGraphEventArray Prereqs;
			Prereqs.Add(State->FusionTableSeededEvent);
			Private::RunOrDispatchIndexWorker(
				[State, MergeAndRank]() { MergeAndRank(State); },
				&Prereqs);
		}
	};


	if (bWantsVector)
	{
		FGraphEventRef VectorIndexReadEvent;
		if (OutIndexReadEvents)
		{
			VectorIndexReadEvent = FGraphEvent::CreateGraphEvent();
			OutIndexReadEvents->Add(VectorIndexReadEvent);
		}
		VectorIndex->Search(QueryEmbedding, FetchK, SharedIDFilter, DistanceCutoff, State->VectorScratch,
			MoveTemp(VectorIndexReadEvent),
			[State, OnArrival](TArray<FSearchResult>&& Out)
			{
				State->VectorResults = MoveTemp(Out);
				OnArrival(State, /*bIsVector*/ true);
			});
	}

	if (bWantsText)
	{
		FGraphEventRef BM25IndexReadEvent;
		if (OutIndexReadEvents)
		{
			BM25IndexReadEvent = FGraphEvent::CreateGraphEvent();
			OutIndexReadEvents->Add(BM25IndexReadEvent);
		}
		// TODO: Move into game thread safe function
		const int32 MinShouldMatchPercent = Settings ? Settings->BM25MinShouldMatchPercent : 75;
		BM25Index->Search(QueryText, FetchK, SharedIDFilter, State->TextScratch,
			MoveTemp(BM25IndexReadEvent),
			MinShouldMatchPercent,
			[State, OnArrival](TArray<FBM25Result>&& Out)
			{
				State->TextResults = MoveTemp(Out);
				OnArrival(State, /*bIsVector*/ false);
			});
	}
}

FString FHybridSearchIndex::GetCodebookPath(ESemanticSearchIndexType Type)
{
	const FString Filename = GetCodebookFilename(Type);
	if (Filename.IsEmpty())
	{
		return FString();
	}
	return FPaths::ProjectConfigDir() / TEXT("SemanticSearch") / Filename;
}

TArray64<uint8> FHybridSearchIndex::SerializeIndexForSave(const TMap<int64, FIoHash>& PackageHashes) const
{
	TArray64<uint8> FileData;
	FMemoryWriter64 Ar(FileData);

	uint32 WriteMagic = FHybridIndexHeader::Magic;
	uint32 WriteVersion = FHybridIndexHeader::Version;
	Ar << WriteMagic << WriteVersion;

	bool bWriteInitialized = bInitialized.load(std::memory_order_relaxed);
	Ar << bWriteInitialized;

	// Vector index type + codebook hash
	const bool bInit = bInitialized.load(std::memory_order_relaxed);
	{
		const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
		uint8 IndexType = (bInit && VectorIndex && Settings)
			? static_cast<uint8>(Settings->IndexType)
			: static_cast<uint8>(ESemanticSearchIndexType::Flat);
		Ar << IndexType;

		FIoHash CodebookHash;
		if (bInit && VectorIndex && VectorIndex->SupportsQuantization() && VectorIndex->IsTrained())
		{
			CodebookHash = static_cast<IQuantizedVectorIndex*>(VectorIndex.Get())->GetCodebookHash();
		}
		Ar << CodebookHash;
	}

	if (bInit && VectorIndex)
	{
		int32 VectorDim = VectorIndex->GetDimension();
		TArray<uint8> VectorData = VectorIndex->Serialize();
		Ar << VectorDim;
		Ar << VectorData;
	}
	else
	{
		int32 VectorDim = 0;
		TArray<uint8> VectorData;
		Ar << VectorDim;
		Ar << VectorData;
	}

	// BM25 index
	BM25Index->Serialize(Ar);

	// Staleness manifest
	TMap<int64, FIoHash> PackageHashesCopy = PackageHashes;
	Ar << PackageHashesCopy;

	return FileData;
}

TArray64<uint8> FHybridSearchIndex::SerializePreProcessorFailedForSave() const
{
	TArray<int64> Array = PreProcessorFailedAssets.Array();

	TArray64<uint8> FileData;
	FMemoryWriter64 Ar(FileData);
	Ar << Array;

	return FileData;
}

void FHybridSearchIndex::WriteBytesToFile(const FString& FilePath, TArray64<uint8> Bytes)
{
	FString Directory = FPaths::GetPath(FilePath);
	IFileManager::Get().MakeDirectory(*Directory, /*bTree=*/true);

	TUniquePtr<FArchive> Writer(IFileManager::Get().CreateFileWriter(*FilePath));
	if (!Writer)
	{
		UE_LOGF(LogSemanticSearch, Warning, "Failed to open semantic-search file for write: %ls", *FilePath);
		return;
	}

	Writer->Serialize(Bytes.GetData(), Bytes.Num());
	if (!Writer->Close())
	{
		UE_LOGF(LogSemanticSearch, Warning, "Failed to write semantic-search file: %ls", *FilePath);
	}
}

void FHybridSearchIndex::LoadPreProcessorFailedFromFile(const FString& FilePath)
{
	// Missing file is the normal state on first run and after a user-initiated reset — stay silent.
	if (!IFileManager::Get().FileExists(*FilePath))
	{
		PreProcessorFailedAssets.Empty();
		return;
	}

	TArray<uint8> FileData;
	if (!FFileHelper::LoadFileToArray(FileData, *FilePath))
	{
		UE_LOGF(LogSemanticSearch, Warning, "Failed to read pre-processor failed sidecar file: %ls", *FilePath);
		PreProcessorFailedAssets.Empty();
		return;
	}

	FMemoryReader Ar(FileData);
	TArray<int64> Array;
	Ar << Array;
	if (Ar.IsError())
	{
		UE_LOGF(LogSemanticSearch, Warning, "Pre-processor failed sidecar file is corrupt, ignoring: %ls", *FilePath);
		PreProcessorFailedAssets.Empty();
		return;
	}

	PreProcessorFailedAssets.Empty(Array.Num());
	for (int64 ID : Array)
	{
		PreProcessorFailedAssets.Add(ID);
	}
	UpdateCachedStats();
}

bool FHybridSearchIndex::LoadFromFile(const FString& FilePath, TMap<int64, FIoHash>& OutPackageHashes)
{
	if (!FPaths::FileExists(FilePath))
	{
		return false;
	}

	TUniquePtr<FArchive> ArPtr(IFileManager::Get().CreateFileReader(*FilePath));
	if (!ArPtr)
	{
		return false;
	}
	FArchive& Ar = *ArPtr;

	uint32 Magic = 0;
	uint32 Version = 0;
	Ar << Magic << Version;
	if (Magic != FHybridIndexHeader::Magic || Version != FHybridIndexHeader::Version)
	{
		return false;
	}

	// Deserialize into temporaries  - only swap into members on full success
	bool bLoadedInitialized = false;
	Ar << bLoadedInitialized;

	uint8 IndexType = 0;
	Ar << IndexType;

	FIoHash SavedCodebookHash;
	Ar << SavedCodebookHash;

	int32 VectorDim = 0;
	TArray<uint8> VectorData;
	Ar << VectorDim;
	Ar << VectorData;

	if (Ar.IsError())
	{
		return false;
	}

	TSharedPtr<IVectorIndex> LoadedVectorIndex;
	if (bLoadedInitialized && VectorDim > 0 && VectorData.Num() > 0)
	{
		const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
		const ESemanticSearchIndexType SavedType = static_cast<ESemanticSearchIndexType>(IndexType);
		const ESemanticSearchIndexType CurrentType = Settings ? Settings->IndexType : ESemanticSearchIndexType::Flat;

		if (SavedType != CurrentType)
		{
			UE_LOGF(LogSemanticSearch, Warning, "Saved index type (%d) does not match current settings (%d), discarding",
				IndexType, static_cast<uint8>(CurrentType));
			return false;
		}

		// Validate codebook hash for compressed index types
		if (SavedType != ESemanticSearchIndexType::Flat && SavedCodebookHash != FIoHash())
		{
			const FString CodebookPath = GetCodebookPath(SavedType);
			TArray<uint8> CodebookData;
			if (FFileHelper::LoadFileToArray(CodebookData, *CodebookPath))
			{
				TSharedPtr<IVectorIndex> TempIndex = DeserializeCodebook(SavedType, CodebookData, VectorDim);
				if (TempIndex && TempIndex->SupportsQuantization())
				{
					FIoHash CurrentCodebookHash = static_cast<IQuantizedVectorIndex*>(TempIndex.Get())->GetCodebookHash();
					if (SavedCodebookHash != CurrentCodebookHash)
					{
						UE_LOGF(LogSemanticSearch, Warning,
							"Saved index codebook hash does not match current codebook, discarding saved index");
						return false;
					}
				}
			}
		}

		LoadedVectorIndex = DeserializeVectorIndex(SavedType, VectorData, VectorDim);

		if (!LoadedVectorIndex)
		{
			return false;
		}
	}

	// BM25 index
	TUniquePtr<FBM25Index> LoadedBM25 = FBM25Index::Deserialize(Ar);
	if (!LoadedBM25)
	{
		return false;
	}

	// Staleness manifest
	TMap<int64, FIoHash> LoadedHashes;
	Ar << LoadedHashes;

	if (Ar.IsError())
	{
		return false;
	}

	// All deserialization succeeded  - swap into members
	bInitialized.store(bLoadedInitialized, std::memory_order_release);
	VectorIndex = MoveTemp(LoadedVectorIndex);
	BM25Index = MoveTemp(LoadedBM25);
	OutPackageHashes = MoveTemp(LoadedHashes);

	UpdateCachedStats();

	return true;
}

} // namespace UE::SemanticSearch
