// Copyright Epic Games, Inc. All Rights Reserved.

#include "Implementations/SemanticSearchImplementationUtils.h"

#include "HAL/IConsoleManager.h"

namespace UE::SemanticSearch::Private
{

// ---------------------------------------------------------------------------
// Profiling toggles
// ---------------------------------------------------------------------------
//
// These CVars exist to help profile the semantic search indexes. The hot
// search path layers async work in three places:
//   1. The hybrid command queue's consumer worker (drains pending search
//      requests on a background thread).
//   2. PQVectorIndex::Search dispatching its scan onto the task graph.
//   3. ParallelFor / ParallelForWithTaskContext fanning out per-doc work
//      inside FBM25Index::Search and FPQVectorIndex::Search, plus the final
//      RadixSort permute.
//
// Layer (3) is fine-grained and runs while (2) is on the wire — great for
// throughput, but it makes flat profiler traces hard to read because per-stage
// cost is smeared across worker threads.
//
// Two CVars give you knobs to peel back the layers one at a time:
//
//   * SemanticSearch.ForceSequentialSemanticSearchIndexWorker  (default 0)
//       When 1, task-graph dispatches inside the index Search functions are
//       inlined instead of scheduled — the task body runs on the calling
//       thread. Collapses layer (2) so search work shows up under the calling
//       stack frame.
//
//   * SemanticSearch.KeepParallelFor             (default 1)
//       When 0, ParallelFor / ParallelForWithTaskContext calls in the search
//       path are issued with EParallelForFlags::ForceSingleThread — they run
//       all iterations on the calling thread. Collapses layer (3).
//
// Useful combinations for profiling:
//   * (0, 1) — production: everything parallel. Fastest wall time, busiest
//              flame graph.
//   * (1, 1) — task graph collapsed but inner loops still parallel. Good for
//              isolating per-stage costs (search vs filter vs scoring) while
//              keeping the loops fast enough to run a representative query.
//   * (1, 0) — fully serial. Apples-to-apples baseline against an O(N) hand
//              count. Slow, but shows you which stages actually scale.
//
// Both CVars are read once per call site via GetValueOnAnyThread() so toggling
// at runtime via the console takes effect on the next search.
// ---------------------------------------------------------------------------

static TAutoConsoleVariable<int32> CVarForceSequentialIndexWorker(
	TEXT("SemanticSearch.ForceSequentialSemanticSearchIndexWorker"),
	0,
	TEXT("Profiling aid. When non-zero, semantic-search index workers run on the calling thread instead of being dispatched through the task graph. ")
	TEXT("Use to flatten the per-stage cost in a profiler trace. Combine with SemanticSearch.KeepParallelFor=0 for a fully serial baseline. Default: 0."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarKeepParallelFor(
	TEXT("SemanticSearch.KeepParallelFor"),
	1,
	TEXT("Profiling aid. When zero, ParallelFor / ParallelForWithTaskContext calls inside the search path are forced single-thread. ")
	TEXT("Use with SemanticSearch.ForceSequentialSemanticSearchIndexWorker=1 to get a flat synchronous trace of search internals. Default: 1 (parallel)."),
	ECVF_Default);

bool ShouldForceSequentialIndexWorker()
{
	return CVarForceSequentialIndexWorker.GetValueOnAnyThread() != 0;
}

bool ShouldKeepParallelFor()
{
	return CVarKeepParallelFor.GetValueOnAnyThread() != 0;
}

EParallelForFlags GetSearchParallelForFlags(EParallelForFlags BaseFlags)
{
	if (!ShouldKeepParallelFor())
	{
		BaseFlags |= EParallelForFlags::ForceSingleThread;
	}
	return BaseFlags;
}

void RunOrDispatchIndexWorker(
	TUniqueFunction<void()> Body,
	const FGraphEventArray* Prereqs,
	ENamedThreads::Type DesiredThread)
{
	if (ShouldForceSequentialIndexWorker())
	{
		Body();
	}
	else
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady(
			MoveTemp(Body),
			TStatId{},
			Prereqs,
			DesiredThread);
	}
}

} // namespace UE::SemanticSearch::Private
