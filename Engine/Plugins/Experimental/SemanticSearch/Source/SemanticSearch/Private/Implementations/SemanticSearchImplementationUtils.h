// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Sort.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/UnrealMemory.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

namespace UE::SemanticSearch::Private
{

// ---------------------------------------------------------------------------
// Profiling toggles for the semantic-search hot path.
//
//
//   * SemanticSearch.ForceSequentialSemanticSearchIndexWorker — when on, task-graph
//     dispatches inside the index Search functions run inline on the calling
//     thread. See SemanticSearchImplementationUtils.cpp for usage notes.
//
//   * SemanticSearch.KeepParallelFor — when off, ParallelFor /
//     ParallelForWithTaskContext calls in the search path fall back to single
//     thread via EParallelForFlags::ForceSingleThread.
//
// Default values keep the production parallel structure intact.
// ---------------------------------------------------------------------------

/** Returns true while SemanticSearch.ForceSequentialSemanticSearchIndexWorker is non-zero. */
bool ShouldForceSequentialIndexWorker();

/** Returns true while SemanticSearch.KeepParallelFor is non-zero (the default). */
bool ShouldKeepParallelFor();

/**
 * Returns BaseFlags with EParallelForFlags::ForceSingleThread OR'd in when
 * SemanticSearch.KeepParallelFor is off. Use at every ParallelFor call site
 * in the search path so the CVar can flatten the trace.
 */
EParallelForFlags GetSearchParallelForFlags(EParallelForFlags BaseFlags = EParallelForFlags::None);

/**
 * Either runs Body inline (when SemanticSearch.ForceSequentialSemanticSearchIndexWorker
 * is non-zero) or dispatches it onto the task graph with the supplied prereqs.
 *
 * In force-sequential mode Prereqs is ignored by design: upstream stages have
 * already run synchronously on the calling thread, so the dependency is
 * trivially satisfied.
 */
void RunOrDispatchIndexWorker(
	TUniqueFunction<void()> Body,
	const FGraphEventArray* Prereqs = nullptr,
	ENamedThreads::Type DesiredThread = ENamedThreads::AnyBackgroundHiPriTask);

/**
 * In-place radix sort of an item array by a non-negative float key.
 *
 * Why this exists:
 *   The default Algo::Sort (introsort) is O(N log N) with random-access compare-
 *   and-swap and gets cache-unfriendly past a few hundred thousand items.
 *
 * Restriction:
 *   - Key must be a NON-NEGATIVE float. 
 *
 * @param InOut         Items to sort, in place. Must be trivially copyable.
 * @param Scratch       Caller-owned scratch buffer, all packed into one
 *                      TArray<uint32>:
 *                        [ Counts        : 256·NumStripes ]
 *                        [ ItemsScratch  : ceil(N·sizeof(TItem)/4) ].
 * @param KeyAccessor   Callable mapping `const TItem&` → `float`. Must return a
 *                      non-negative value.
 * @param bDescending   When true, sort by descending key.
 */
template <typename TItem>
void RadixSort(TArray<TItem>& InOut, TArray<uint32>& Scratch, auto KeyAccessor, bool bDescending = false)
{
	// The radix passes write items via raw byte copy and overwrite InOut /
	// the reinterpreted Scratch tail without running destructors. Both
	// require POD-like TItem.
	static_assert(std::is_trivially_copyable_v<TItem>,
		"RadixSort scatters items via byte copy. TItem must be trivially copyable.");
	static_assert(std::is_trivially_destructible_v<TItem>,
		"RadixSort overwrites items via byte copy without running destructors. "
		"TItem must be trivially destructible.");

	// ItemsScratch starts at byte offset 1024·NumStripes from Scratch.GetData(),
	// which is 16-byte aligned. Covers TItem alignof up to 16
	static_assert(alignof(TItem) <= 16,
		"RadixSort's ItemsScratch region is 16-byte aligned. "
		"TItem alignof exceeds that — extend the layout to add padding.");

	const int32 N = InOut.Num();

	constexpr int32 RadixThreshold = 1024;
	if (N < RadixThreshold)
	{
		if (bDescending)
		{
			InOut.Sort([&KeyAccessor](const TItem& A, const TItem& B)
			{
				return KeyAccessor(A) > KeyAccessor(B);
			});
		}
		else
		{
			InOut.Sort([&KeyAccessor](const TItem& A, const TItem& B)
			{
				return KeyAccessor(A) < KeyAccessor(B);
			});
		}
		return;
	}

	// Compute the parallel layout up front so all radix scratch can be fused
	// into a single allocation.
	const int32 NumWorkers = FMath::Max(1, static_cast<int32>(LowLevelTasks::FScheduler::Get().GetNumWorkers()));
	const int32 StripeSize = FMath::Max(8192, FMath::DivideAndRoundUp(N, NumWorkers));
	const int32 NumStripes = FMath::Max(1, FMath::DivideAndRoundUp(N, StripeSize));

	// All radix bookkeeping AND the item ping-pong buffer live in the
	// caller-supplied Scratch buffer, contiguous in memory:
	//   [ Counts: 256×NumStripes ][ ItemsScratch ]
	const size_t ItemUInt32CountWide =
		(static_cast<size_t>(N) * sizeof(TItem) + sizeof(uint32) - 1) / sizeof(uint32);
	const size_t RequiredScratchSizeWide =
		static_cast<size_t>(256) * static_cast<size_t>(NumStripes) + ItemUInt32CountWide;
	checkf(RequiredScratchSizeWide <= static_cast<size_t>(MAX_int32),
		TEXT("RadixSort scratch size %llu uint32s exceeds int32 (N=%d, sizeof(TItem)=%llu, NumStripes=%d)."),
		static_cast<uint64>(RequiredScratchSizeWide),
		N,
		static_cast<uint64>(sizeof(TItem)),
		NumStripes);
	const int32 ItemUInt32Count = static_cast<int32>(ItemUInt32CountWide);
	const int32 RequiredScratchSize = static_cast<int32>(RequiredScratchSizeWide);
	Scratch.SetNumUninitialized(RequiredScratchSize, EAllowShrinking::No);
	const TArrayView<uint32> ScratchView(Scratch);
	TArrayView<uint32> LocalCounts = ScratchView.Slice(0, 256 * NumStripes);

	// Reinterpret the trailing uint32s as raw TItem storage. Offset
	// 1024·NumStripes bytes is 16-byte aligned.
	TItem* const ItemsScratchPtr = reinterpret_cast<TItem*>(Scratch.GetData() + 256 * NumStripes);

	// Item ping-pong endpoints.
	TItem* SrcItems = InOut.GetData();
	TItem* DstItems = ItemsScratchPtr;

	// For descending: XOR every extracted key with 0xFFFFFFFF before binning.
	// Ascending uint32 radix on the inverted key bits produces descending float order. 
	const uint32 KeyXor = bDescending ? 0xFFFFFFFFu : 0u;

	// 4 byte passes over the float bits. Each pass:
	//   A. Histogram: KeyAccessor each item, bin into stripe-local 256-bucket
	//      counts.
	//   B. Sequential transpose + prefix-sum on LocalCounts. Walk digits
	//      outer, stripes inner — each cell becomes that (stripe, digit)'s
	//      exclusive output offset in DstItems.
	//   C. Scatter: each stripe re-extracts the key, reads its slice of
	//      source items, and writes to DstItems at the prefix-sum'd
	//      position.
	check(LocalCounts.Num() == 256 * NumStripes);
	for (int32 Pass = 0; Pass < 4; ++Pass)
	{
		const int32 Shift = Pass * 8;

		// Re-cache raw pointers each pass — Swap rotates the item endpoints.
		TItem*  const SrcItemsData    = SrcItems;
		TItem*  const DstItemsData    = DstItems;
		uint32* const LocalCountsData = LocalCounts.GetData();
		const int32 LocalCountsNum    = LocalCounts.Num();

		FMemory::Memzero(LocalCountsData, LocalCountsNum * sizeof(uint32));

		ParallelFor(
			TEXT("RadixSort Histogram"),
			NumStripes,
			1,
			[N, Shift, StripeSize, SrcItemsData, &KeyAccessor, LocalCountsData, KeyXor](int32 s)
			{
				const int32 Begin = s * StripeSize;
				const int32 End = FMath::Min(Begin + StripeSize, N);
				uint32* const C = LocalCountsData + 256 * s;
				for (int32 i = Begin; i < End; ++i)
				{
					const float Key = KeyAccessor(SrcItemsData[i]);
					uint32 KeyBits;
					FMemory::Memcpy(&KeyBits, &Key, sizeof(uint32));
					KeyBits ^= KeyXor;
					++C[(KeyBits >> Shift) & 0xFFu];
				}
			},
			GetSearchParallelForFlags());

		uint32 Sum = 0;
		for (int32 d = 0; d < 256; ++d)
		{
			for (int32 s = 0; s < NumStripes; ++s)
			{
				uint32& Slot = LocalCountsData[s * 256 + d];
				const uint32 c = Slot;
				Slot = Sum;
				Sum += c;
			}
		}

		ParallelFor(
			TEXT("RadixSort Scatter"),
			NumStripes,
			1,
			[N, Shift, StripeSize, SrcItemsData, DstItemsData, &KeyAccessor, LocalCountsData, KeyXor](int32 s)
			{
				const int32 Begin = s * StripeSize;
				const int32 End = FMath::Min(Begin + StripeSize, N);
				uint32* const C = LocalCountsData + 256 * s;
				for (int32 i = Begin; i < End; ++i)
				{
					const float Key = KeyAccessor(SrcItemsData[i]);
					uint32 KeyBits;
					FMemory::Memcpy(&KeyBits, &Key, sizeof(uint32));
					KeyBits ^= KeyXor;
					const uint32 d = (KeyBits >> Shift) & 0xFFu;
					const uint32 Pos = C[d]++;
					DstItemsData[Pos] = SrcItemsData[i];
				}
			},
			GetSearchParallelForFlags());

		Swap(SrcItems, DstItems);
	}

	// After 4 (even) passes, the final scatter wrote to InOut.
}

} // namespace UE::SemanticSearch::Private
