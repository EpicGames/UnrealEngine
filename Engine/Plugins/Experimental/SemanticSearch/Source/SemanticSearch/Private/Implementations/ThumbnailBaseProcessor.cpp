// Copyright Epic Games, Inc. All Rights Reserved.

#include "Implementations/ThumbnailBaseProcessor.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "ImageCore.h"
#include "Misc/Guid.h"
#include "Misc/ObjectThumbnail.inl"
#include "Misc/PackageName.h"
#include "ObjectTools.h"

namespace UE::SemanticSearch
{

bool FThumbnailBaseAssetProcessor::GenerateAssetHash(const FAssetData& Asset, FMemoryHasherBlake3& InHasher) const
{
	if (Asset.HasAnyPackageFlags(PKG_Cooked))
	{
		return false;
	}

	FAssetPackageData AssetPackageData;
	if (IAssetRegistry::GetChecked().TryGetAssetPackageData(Asset.PackageName, AssetPackageData) != AssetRegistry::EExists::Exists)
	{
		return false;
	}

	constexpr FGuid DerivedDataGuid = FGuid(0x47702F04, 0x21B445B6, 0x96DE9103, 0xC16D689E);

	// Safe we know the hasher won't modify it
	InHasher << const_cast<FGuid&>(DerivedDataGuid);

	FName PackageName(Asset.PackageName);
	InHasher << PackageName;

	FName AssetName(Asset.AssetName);
	InHasher << AssetName;

	FIoHash PackageHash = AssetPackageData.GetPackageSavedHash();
	InHasher << PackageHash;

	int32 Revision = GetRevision();
	InHasher << Revision;

	return true;
}

void FThumbnailBaseAssetProcessor::GenerateCaptionRequest(const TSharedRef<const FAssetData>& InAsset, bool bCanDisruptUser, DerivedData::FRequestOwner& RequestOwner, FOnRequestComplete OnRequestComplete) const
{
	if (IsEngineExitRequested())
	{
		OnRequestComplete(false, FCaptionRequest(), FString(), EAssetIndexFailureReason::None);
		return;
	}

	FString PackagePath;
	if (FPackageName::DoesPackageExist(InAsset->PackageName.ToString(), &PackagePath))
	{
		TSet<FName> ObjectsFullName;
		FName ObjectFullName = *InAsset->GetFullName();
		ObjectsFullName.Add(ObjectFullName);

		FThumbnailMap ThumbnailMap;
		// Don't use the cache to avoid pollution. Consider revisiting this?
		if (ThumbnailTools::LoadThumbnailsFromPackageDirectly(PackagePath, ObjectsFullName, ThumbnailMap, PackagePath))
		{
			if (FObjectThumbnail* Thumbnail = ThumbnailMap.Find(ObjectFullName))
			{
				FCaptionRequest CaptionRequest;

				// Convert the format from BGRA8 to RGBA8 inplace
				Thumbnail->DecompressImageData();
				if (!ensure(Thumbnail->GetImage().Format == ERawImageFormat::BGRA8))
				{
					OnRequestComplete(false, FCaptionRequest(), FString(TEXT("Thumbnail was using an unexpected data format.")), EAssetIndexFailureReason::PreProcessor);
					return;
				}
				TArrayView<uint8> UncompressedImage = Thumbnail->AccessImageData();
				for (int32 PixelIndex = 0; PixelIndex < UncompressedImage.Num(); PixelIndex += 4)
				{
					constexpr int32 RedOffsetInBGRA8 = 2;
					Swap(UncompressedImage[PixelIndex], UncompressedImage[PixelIndex + RedOffsetInBGRA8]);
				}
				Thumbnail->CompressImageData();
				Thumbnail->AccessImageData().Empty();

				FThumbnailCompressionInterface* Compressor = Thumbnail->GetCompressor();
				if (!Compressor)
				{
					OnRequestComplete(false, FCaptionRequest(), FString(TEXT("Thumbnail compressor is null")), EAssetIndexFailureReason::PreProcessor);
					return;
				}

				FAssetMedia Media;
				Media.MimeType = Compressor->GetMimeType();
				Media.Data = MoveTemp(Thumbnail->AccessCompressedImageData());
				CaptionRequest.AssetMedia.Add(MoveTemp(Media));

				CaptionRequest.AssetPath = InAsset->GetObjectPathString();
				CaptionRequest.AssetType = InAsset->AssetClassPath.ToString();

				CaptionRequest.Metadata = GetMetadata(InAsset);

				OnRequestComplete(true, MoveTemp(CaptionRequest), FString(), EAssetIndexFailureReason::None);
				return;
			}
		}
	}

	OnRequestComplete(false, FCaptionRequest(), FString(TEXT("Couldn't fetch an on-disk thumbnail for the asset")), EAssetIndexFailureReason::PreProcessor);
}

}
