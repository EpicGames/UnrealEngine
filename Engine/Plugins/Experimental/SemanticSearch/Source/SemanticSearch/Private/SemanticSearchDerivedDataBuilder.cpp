// Copyright Epic Games, Inc. All Rights Reserved.

#include "SemanticSearchDerivedDataBuilder.h"

#include "AssetProcessorManager.h"
#include "Containers/SharedString.h"
#include "HybridSearchIndex.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Containers/StringFwd.h"
#include "DerivedDataCache.h"
#include "Memory/SharedBuffer.h"
#include "Interfaces/IAssetProcessor.h"
#include "Misc/Guid.h"
#include "Misc/ScopeRWLock.h"
#include "SemanticSearchModule.h"
#include "Serialization/MemoryHasher.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "String/Find.h"
#include "Misc/StringBuilder.h"

namespace UE::SemanticSearch::Private
{

// Cap the number of DDC requests per GetCache().Get() call to avoid Zen server timeouts
// on very large batches. Requests above this threshold are split into multiple chunks.
constexpr int32 MaxDDCBatchSize = 5000;

DerivedData::FValueId GetEmbeddingDatachunkId()
{
	return DerivedData::FValueId::FromName(TEXT("EmbeddingData"));
}

DerivedData::FValueId GetCaptionDatachunkId()
{
	return DerivedData::FValueId::FromName(TEXT("CaptionData"));
}

DerivedData::FValueId GetQuantizedEmbeddingDatachunkId()
{
	return DerivedData::FValueId::FromName(TEXT("QuantizedEmbeddingData"));
}

void SerializeEmbedding(FEmbeddingDerivedData& Embedding, FArchive& Archive)
{
	Archive << Embedding.Embedding;
}

void SerializeQuantizedEmbedding(FQuantizedEmbeddingDerivedData& QuantizedEmbedding, FArchive& Archive)
{
	Archive << QuantizedEmbedding.QuantizedEmbedding;
}

void SerializeCaption(FCaptionDerivedData& Caption, FArchive& Archive)
{
	Archive << Caption.Caption;
	Archive << Caption.Keywords;
}

DerivedData::ECachePolicy GetCachePolicy()
{
	using namespace DerivedData;
	// We will need to revisit the non deterministic part of that if we are to cook the embedding data in a cook editor at some point.
	return ECachePolicy::Default | ECachePolicy::NonDeterministic;
}

TSharedPtr<FSemanticSearchBuildCacheTask> FSemanticSearchBuildCacheTask::Create(const FAssetData& InAssetData, const TSharedRef<IAssetProcessor>& InAssetProcessor, DerivedData::FCacheKey&& InKey)
{
	return MakeShared<FSemanticSearchBuildCacheTask>(FAssetData(InAssetData), InAssetProcessor, MoveTemp(InKey));
}

TSharedPtr<FSemanticSearchBuildCacheTask> FSemanticSearchBuildCacheTask::Create(FAssetData&& InAssetData, const TSharedRef<IAssetProcessor>& InAssetProcessor, DerivedData::FCacheKey&& InKey)
{
	return MakeShared<FSemanticSearchBuildCacheTask>(MoveTemp(InAssetData), InAssetProcessor, MoveTemp(InKey));
}


FSemanticSearchBuildCacheTask::FSemanticSearchBuildCacheTask(FAssetData&& InAssetData, const TSharedRef<IAssetProcessor>& InAssetProcessor, DerivedData::FCacheKey&& InAssetKey)
	: DerivedData::FRequestOwner(DerivedData::EPriority::Low)
	, Asset(MakeShared<FAssetData>(MoveTemp(InAssetData)))
	, AssetProcessor(InAssetProcessor)
	, AssetKey(MoveTemp(InAssetKey))
	, DDCRequestName(FString::Format(TEXT("DDCSemanticSearch:%s"), {FStringFormatArg(Asset->GetObjectPathString())}))
{
}

void FSemanticSearchBuildCacheTask::GetEmbeddingData(TUniqueFunction<void(FEmbeddingDerivedData&& EmbeddingData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)>&& InOnDataAvailable, bool bBuildOnMiss)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetEmbeddingData");
	{
		FReadScopeLock ReadLock(IsBuildingDerivedDataLock);
		QueuedEmbeddingRequests.Enqueue(MoveTemp(InOnDataAvailable));
		bool bPreviousValue = false;
		HasActiveEmbeddingRequest.compare_exchange_strong(bPreviousValue, true);
		if (bIsBuildingDerivedData || bPreviousValue)
		{
			return;
		}
	}

	using namespace DerivedData;

	FCacheGetChunkRequest Request;
	Request.Name = DDCRequestName;
	Request.Key = AssetKey;
	Request.Id = GetEmbeddingDatachunkId();
	Request.Policy = GetCachePolicy();

	FRequestBarrier Barrier(*this);
	GetCache().GetChunks({ Request }, *this, [this, bBuildOnMiss, SelfKeepAlive = AsShared()](FCacheGetChunkResponse&& Response) mutable
		{
			if (Response.Status == EStatus::Ok && Response.RawData.GetSize() > 0)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetEmbeddingData::DDCHit");
				FEmbeddingDerivedData Data;

				FMemoryReaderView Reader(Response.RawData.GetView(), true);
				SerializeEmbedding(Data, Reader);
				Response.RawData.Reset();

				BroadcastQueuedEmbeddingRequest(MoveTemp(Data), FString(), EAssetIndexFailureReason::None);
			}
			else if (Response.Status != EStatus::Canceled)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetEmbeddingData::DDCMiss");
				if (bBuildOnMiss)
				{
					this->BuildDerivedData();
				}
				else
				{
					// Cache miss without a build fallback — not a failure to mark.
					BroadcastQueuedEmbeddingRequest(FEmbeddingDerivedData(), FString(TEXT("Not cached")), EAssetIndexFailureReason::None);
				}
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetEmbeddingData::DDCCanceled");
				BroadcastQueuedEmbeddingRequest(FEmbeddingDerivedData(), FString(TEXT("The data request was cancelled")), EAssetIndexFailureReason::None);
			}
		});
}

void FSemanticSearchBuildCacheTask::GetCaptionData(TUniqueFunction<void(FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)>&& InOnDataAvailable)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetCaptionData");
	{
		FReadScopeLock ReadLock(IsBuildingDerivedDataLock);
		QueuedCaptionRequests.Enqueue(MoveTemp(InOnDataAvailable));
		bool bPreviousValue = false;
		HasActiveCaptionRequest.compare_exchange_strong(bPreviousValue, true);
		if (bIsBuildingDerivedData || bPreviousValue)
		{
			return;
		}
	}

	using namespace DerivedData;
	FCacheGetChunkRequest Request;
	Request.Name = DDCRequestName;
	Request.Key = AssetKey;
	Request.Id = GetCaptionDatachunkId();
	Request.Policy = GetCachePolicy();

	FRequestBarrier Barrier(*this);
	GetCache().GetChunks({ Request }, *this, [this, SelfKeepAlive = AsShared()](FCacheGetChunkResponse&& Response) mutable
		{
			if (Response.Status == EStatus::Ok && Response.RawData.GetSize() > 0)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetCaptionData::DDCHit");
				FCaptionDerivedData Data;
				FMemoryReaderView Reader(Response.RawData.GetView(), true);
				SerializeCaption(Data, Reader);
				Response.RawData.Reset();

				BroadcastQueuedCaptionRequest(MoveTemp(Data), FString(), EAssetIndexFailureReason::None);
			}
			else if (Response.Status != EStatus::Canceled)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetCaptionData::DDCMiss");
				BuildDerivedData();
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetCaptionData::DDCCanceled");
				BroadcastQueuedCaptionRequest(FCaptionDerivedData(), FString(TEXT("The data request was cancelled")), EAssetIndexFailureReason::None);
			}
		});
}

void FSemanticSearchBuildCacheTask::BuildCaptionData(TUniqueFunction<void(FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildCaptionData");
	{
		FReadScopeLock ReadLock(IsBuildingDerivedDataLock);
		QueuedCaptionRequests.Enqueue(MoveTemp(InOnDataAvailable));
		bool bPreviousValue = false;
		HasActiveCaptionRequest.compare_exchange_strong(bPreviousValue, true);
		if (bIsBuildingDerivedData || bPreviousValue)
		{
			return;
		}
	}

	BuildDerivedData();
}

void FSemanticSearchBuildCacheTask::BuildRecordData(TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildRecordData");
	{
		FReadScopeLock ReadLock(IsBuildingDerivedDataLock);
		QueuedRecordRequests.Enqueue(MoveTemp(InOnDataAvailable));
		bool bPreviousValue = false;
		HasActiveRecordRequest.compare_exchange_strong(bPreviousValue, true);
		if (bIsBuildingDerivedData || bPreviousValue)
		{
			return;
		}
	}

	BuildDerivedData();
}

void FSemanticSearchBuildCacheTask::BuildQuantizedEmbeddingData(const FIoHash& CodebookHash, TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildQuantizedEmbeddingData");
	{
		QueuedQuantizedRequests.Enqueue(MoveTemp(InOnDataAvailable));
		bool bPreviousValue = false;
		HasActiveQuantizedRequest.compare_exchange_strong(bPreviousValue, true);
		if (bPreviousValue)
		{
			return;
		}
	}

	BuildQuantizedDerivedData(CodebookHash, /*bBuildOnMiss=*/true);
}

void FSemanticSearchBuildCacheTask::GetQuantizedEmbeddingData(
	const FIoHash& CodebookHash,
	TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable,
	bool bBuildOnMiss)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetQuantizedEmbeddingData");

	{
		QueuedQuantizedRequests.Enqueue(MoveTemp(InOnDataAvailable));
		bool bPreviousValue = false;
		HasActiveQuantizedRequest.compare_exchange_strong(bPreviousValue, true);
		if (bPreviousValue)
		{
			return;
		}
	}

	DerivedData::FCacheKey QuantizedKey = GenerateQuantizedCacheKey(AssetKey, CodebookHash);

	using namespace DerivedData;

	FCacheGetChunkRequest Request;
	Request.Name = DDCRequestName;
	Request.Key = QuantizedKey;
	Request.Id = GetQuantizedEmbeddingDatachunkId();
	Request.Policy = GetCachePolicy();

	FRequestBarrier Barrier(*this);
	GetCache().GetChunks({ Request }, *this,
		[this, CodebookHash, bBuildOnMiss, SelfKeepAlive = AsShared()](FCacheGetChunkResponse&& Response) mutable
		{
			if (Response.Status == EStatus::Ok && Response.RawData.GetSize() > 0)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetQuantizedEmbeddingData::DDCHit");
				FQuantizedEmbeddingDerivedData Data;
				FMemoryReaderView Reader(Response.RawData.GetView(), true);
				SerializeQuantizedEmbedding(Data, Reader);
				Response.RawData.Reset();
				BroadcastQueuedQuantizedRequest(MoveTemp(Data), FString(), EAssetIndexFailureReason::None);
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetQuantizedEmbeddingData::DDCMiss");
				BuildQuantizedDerivedData(CodebookHash, bBuildOnMiss);
			}
		});
}

void FSemanticSearchBuildCacheTask::GetRecordData(
	TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable,
	bool bBuildOnMiss)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetRecordData");
	{
		FReadScopeLock ReadLock(IsBuildingDerivedDataLock);
		QueuedRecordRequests.Enqueue(MoveTemp(InOnDataAvailable));
		bool bPreviousValue = false;
		HasActiveRecordRequest.compare_exchange_strong(bPreviousValue, true);
		if (bIsBuildingDerivedData || bPreviousValue)
		{
			return;
		}
	}

	using namespace DerivedData;

	FCacheGetRequest Request;
	Request.Name = DDCRequestName;
	Request.Key = AssetKey;
	Request.Policy = GetCachePolicy();

	FRequestBarrier Barrier(*this);
	GetCache().Get({ Request }, *this,
		[this, bBuildOnMiss, SelfKeepAlive = AsShared()](FCacheGetResponse&& Response) mutable
		{
			if (Response.Status == EStatus::Ok)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetRecordData::DDCHit");

				FEmbeddingDerivedData EmbeddingData;
				const FValueWithId& EmbeddingValue = Response.Record.GetValue(GetEmbeddingDatachunkId());
				if (EmbeddingValue.HasData())
				{
					FSharedBuffer RawBuffer = EmbeddingValue.GetData().Decompress();
					FMemoryReaderView Reader(RawBuffer.GetView(), true);
					SerializeEmbedding(EmbeddingData, Reader);
				}

				FCaptionDerivedData CaptionData;
				const FValueWithId& CaptionValue = Response.Record.GetValue(GetCaptionDatachunkId());
				if (CaptionValue.HasData())
				{
					FSharedBuffer RawBuffer = CaptionValue.GetData().Decompress();
					FMemoryReaderView Reader(RawBuffer.GetView(), true);
					SerializeCaption(CaptionData, Reader);
				}

				BroadcastQueuedRecordRequest(MoveTemp(EmbeddingData), MoveTemp(CaptionData), FString(), EAssetIndexFailureReason::None);
			}
			else if (Response.Status != EStatus::Canceled)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetRecordData::DDCMiss");
				if (bBuildOnMiss)
				{
					this->BuildDerivedData();
				}
				else
				{
					// Cache miss without a build fallback — not a failure to mark.
					BroadcastQueuedRecordRequest(
						FEmbeddingDerivedData(), FCaptionDerivedData(), FString(TEXT("Not cached")), EAssetIndexFailureReason::None);
				}
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::GetRecordData::DDCCanceled");
				BroadcastQueuedRecordRequest(
					FEmbeddingDerivedData(), FCaptionDerivedData(), FString(TEXT("The data request was cancelled")), EAssetIndexFailureReason::None);
			}
		});
}

void FSemanticSearchBuildCacheTask::CacheData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::CacheData");
	{
		FReadScopeLock ReadLock(IsBuildingDerivedDataLock);
		if (bIsBuildingDerivedData)
		{
			return;
		}
	}

	using namespace DerivedData;

	FCacheGetRequest Request;
	Request.Name = DDCRequestName;
	Request.Key = AssetKey;
	Request.Policy = GetCachePolicy() | ECachePolicy::SkipData;

	FRequestBarrier Barrier(*this);
	GetCache().Get({ Request }, *this, [this, SelfKeepAlive = AsShared()](FCacheGetResponse&& Response) mutable
		{
			if (Response.Status != EStatus::Ok && Response.Status != EStatus::Canceled)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::CachedData::DDCMiss");
				BuildDerivedData();
			}
		});
}

DerivedData::FCacheKey FSemanticSearchBuildCacheTask::GenerateCacheKey(const FAssetData& InAssetData, const TSharedRef<IAssetProcessor>& InAssetProcessor)
{
	constexpr FGuid SemanticSearchDerivedDataGuid(0x538A7DB0, 0x80304195, 0x84016942, 0xD8879E0A);
	FMemoryHasherBlake3 HashWriter;

	// The Hasher won't modify the value
	HashWriter << const_cast<FGuid&>(SemanticSearchDerivedDataGuid);
	if (!InAssetProcessor->GenerateAssetHash(InAssetData, HashWriter))
	{
		// Invalid Key Generation
		return DerivedData::FCacheKey{};
	}

	TSharedPtr<IEmbeddingProvider> EmbeddingProvider = FSemanticSearchModule::Get().GetEmbeddingProvider();
	if (!EmbeddingProvider)
	{
		return DerivedData::FCacheKey{};
	}
	uint32 ConfigHash = EmbeddingProvider->GetConfigHash();
	HashWriter << ConfigHash;

	DerivedData::FCacheKey CacheKey;
	CacheKey.Hash = HashWriter.Finalize();
	CacheKey.Bucket = GetCacheKeyBucket(InAssetProcessor);
	return CacheKey;
}

DerivedData::FCacheKey FSemanticSearchBuildCacheTask::GenerateQuantizedCacheKey(
	const DerivedData::FCacheKey& BaseCacheKey, const FIoHash& CodebookHash)
{
	FMemoryHasherBlake3 HashWriter;

	// Combine base key hash with codebook hash
	FIoHash BaseHash = BaseCacheKey.Hash;
	HashWriter << BaseHash;
	FIoHash MutableCodebookHash = CodebookHash;
	HashWriter << MutableCodebookHash;

	DerivedData::FCacheKey QuantizedKey;
	QuantizedKey.Hash = HashWriter.Finalize();
	QuantizedKey.Bucket = BaseCacheKey.Bucket;
	return QuantizedKey;
}

DerivedData::FCacheBucket FSemanticSearchBuildCacheTask::GetCacheKeyBucket(const TSharedRef<IAssetProcessor>& InAssetProcessor)
{
	TStringBuilder<256> Builder;
	Builder.Append(TEXTVIEW("SemanticSearch"));
	Builder.Append(InAssetProcessor->GetProcessSubBucketName());
	return DerivedData::FCacheBucket(FStringView(Builder));
}


void FSemanticSearchBuildCacheTask::BuildDerivedData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildDerivedData");
	{
		FWriteScopeLock WriteLock(IsBuildingDerivedDataLock);
		if (bIsBuildingDerivedData)
		{
			return;
		}

		bIsBuildingDerivedData = true;
	}

	DerivedData::FRequestBarrier Barrier(*this);
	AssetProcessor->GenerateCaptionRequest(Asset, false, *this, [this, SelfKeepAlive = AsShared()](bool bHasGeneratedRequest, FCaptionRequest&& CaptionRequest, FString&& ErrorMessage, EAssetIndexFailureReason ProcessorReason)

		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildDerivedData::CaptionRequestReady");
			if (bHasGeneratedRequest)
			{
				if (TSharedPtr<IEmbeddingProvider> EmbeddingProvider = FSemanticSearchModule::Get().GetEmbeddingProvider())
				{
					EmbeddingProvider->GenerateCaptionAsync(MoveTemp(CaptionRequest), [this, EmbeddingProvider, SelfKeepAlive = AsShared()](FCaptionResponse&& Caption)
						{
							TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildDerivedData::CaptionResponseReceived");
							if (Caption.ErrorMessage.IsEmpty())
							{
								FString CaptionString = Caption.Caption;
								EmbeddingProvider->GenerateEmbeddingAsync(CaptionString, [this, UsedCaption = MoveTemp(Caption), SelfKeepAlive = AsShared()](FEmbeddingResponse&& Embedding) mutable
									{
										TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildDerivedData::EmbeddingResponseReceived");
										if (Embedding.ErrorMessage.IsEmpty())
										{
											FCaptionDerivedData CaptionDerivedData;
											CaptionDerivedData.Caption = MoveTemp(UsedCaption.Caption);
											CaptionDerivedData.Keywords = MoveTemp(UsedCaption.Keywords);
											FEmbeddingDerivedData EmbeddingDerivedData;
											EmbeddingDerivedData.Embedding = Embedding.Embedding;

											if (AssetKey == GenerateCacheKey(*Asset, AssetProcessor))
											{
												using namespace DerivedData;

												FCacheRecordBuilder RecordBuilder(AssetKey);

												// Save the caption and keywords
												{
													TArray<uint8> Buffer;
													FMemoryWriter Writer(Buffer);
													SerializeCaption(CaptionDerivedData, Writer);
													RecordBuilder.AddValue(GetCaptionDatachunkId(), MakeSharedBufferFromArray(MoveTemp(Buffer)));
												}

												// Save the embeddings
												{
													TArray<uint8> Buffer;
													FMemoryWriter Writer(Buffer);
													SerializeEmbedding(EmbeddingDerivedData, Writer);
													RecordBuilder.AddValue(GetEmbeddingDatachunkId(), MakeSharedBufferFromArray(MoveTemp(Buffer)));
												}

												DerivedData::FCachePutRequest PutRequest{ DDCRequestName, RecordBuilder.Build(), GetCachePolicy() | ECachePolicy::SkipData };

												FRequestBarrier Barrier(*this);
												DerivedData::GetCache().Put(
													{ PutRequest },
													*this,
													[AssetPtr = Asset](DerivedData::FCachePutResponse&& Response)
													{
														TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildDerivedData::DDCPutResponse");
														if (Response.Status == EStatus::Ok)
														{
															UE_LOGF(LogSemanticSearch, Verbose, "Successfully stored semantic data in DDC for: %ls", *AssetPtr->GetObjectPathString());
														}
														else
														{
															UE_LOGF(LogSemanticSearch, Verbose, "Failed to store data in DDC for: %ls (%ls)", *AssetPtr->GetObjectPathString(), *WriteToString<256>(Response.Status));
														}
													});

												BroadcastQueuedRequests(MoveTemp(EmbeddingDerivedData), MoveTemp(CaptionDerivedData), FString(), EAssetIndexFailureReason::None);
											}
											else
											{
												UE_LOGF(LogSemanticSearch, Verbose, "Data used to generate the semantic data changed while building it for: %ls, Result won't be saved to DDC since they might be inaccurate", *Asset->GetObjectPathString());
												BroadcastQueuedRequests(MoveTemp(EmbeddingDerivedData), MoveTemp(CaptionDerivedData), FString(), EAssetIndexFailureReason::None);
											}
										}
										else
										{
											// Embedding-step failure (provider).
											BroadcastQueuedRequests({}, {}, MoveTemp(Embedding.ErrorMessage), Embedding.FailureReason);
										}

									}, this);
							}
							else
							{
								// Caption-step failure (provider).
								BroadcastQueuedRequests({}, {}, MoveTemp(Caption.ErrorMessage), Caption.FailureReason);
							}
						}, this);
				}
				else
				{
					BroadcastQueuedRequests({}, {}, FString(TEXT("No Embedding provider is active")), EAssetIndexFailureReason::Provider);
				}
			}
			else
			{
				// Pre-processor failure — processor couldn't prepare the caption request.
				BroadcastQueuedRequests({}, {}, MoveTemp(ErrorMessage), ProcessorReason);
			}
		});
}


TUniqueFunction<void(FEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> FSemanticSearchBuildCacheTask::NoLock_GetEmbeddingRequest()
{
	TArray<TUniqueFunction<void(FEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>, TInlineAllocator<4>> Callbacks;
	bool bPreviousValue = true;
	HasActiveEmbeddingRequest.compare_exchange_strong(bPreviousValue, false);
	if (bPreviousValue)
	{
		TUniqueFunction<void(FEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> Callback;
		while (QueuedEmbeddingRequests.Dequeue(Callback))
		{
			Callbacks.Add(MoveTemp(Callback));
		}
	}

	if (Callbacks.IsEmpty())
	{
		return {};
	}

	return [Callbacks = MoveTemp(Callbacks)](FEmbeddingDerivedData&& EmbeddingData, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
	{
		for (int32 Index = 0; Index < Callbacks.Num(); ++Index)
		{
			if (Callbacks[Index])
			{
				if (Index < Callbacks.Num() - 1)
				{
					Callbacks[Index](FEmbeddingDerivedData(EmbeddingData), FString(ErrorMessage), Reason);
				}
				else
				{
					Callbacks[Index](MoveTemp(EmbeddingData), MoveTemp(ErrorMessage), Reason);
				}
			}
		}
	};
}

TUniqueFunction<void(FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> FSemanticSearchBuildCacheTask::NoLock_GetCaptionRequests()
{
	TArray<TUniqueFunction<void(FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>, TInlineAllocator<4>> Callbacks;
	bool bPreviousValue = true;
	HasActiveCaptionRequest.compare_exchange_strong(bPreviousValue, false);
	if (bPreviousValue)
	{
		TUniqueFunction<void(FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> Callback;
		while (QueuedCaptionRequests.Dequeue(Callback))
		{
			Callbacks.Add(MoveTemp(Callback));
		}
	}

	if (Callbacks.IsEmpty())
	{
		return {};
	}

	return [Callbacks = MoveTemp(Callbacks)](FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
	{
		for (int32 Index = 0; Index < Callbacks.Num(); ++Index)
		{
			if (Callbacks[Index])
			{
				if (Index < Callbacks.Num() - 1)
				{
					Callbacks[Index](FCaptionDerivedData(CaptionData), FString(ErrorMessage), Reason);
				}
				else
				{
					Callbacks[Index](MoveTemp(CaptionData), MoveTemp(ErrorMessage), Reason);
				}
			}
		}
	};
}

void FSemanticSearchBuildCacheTask::BroadcastQueuedEmbeddingRequest(FEmbeddingDerivedData&& EmbeddingData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastEmbeddingRequest");
	TUniqueFunction<void(FEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> EmbeddingCallback;
	{
		FWriteScopeLock WriteLock(IsBuildingDerivedDataLock);
		EmbeddingCallback = NoLock_GetEmbeddingRequest();
	}

	if (EmbeddingCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastEmbeddingRequest::Dispatch");
		EmbeddingCallback(MoveTemp(EmbeddingData), MoveTemp(ErrorMessage), Reason);
	}
}

void FSemanticSearchBuildCacheTask::BroadcastQueuedCaptionRequest(FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastCaptionRequest");
	TUniqueFunction<void(FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> CaptionCallback;
	{
		FWriteScopeLock WriteLock(IsBuildingDerivedDataLock);
		CaptionCallback = NoLock_GetCaptionRequests();
	}

	if (CaptionCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastCaptionRequest::Dispatch");
		CaptionCallback(MoveTemp(CaptionData), MoveTemp(ErrorMessage), Reason);
	}
}

void FSemanticSearchBuildCacheTask::BroadcastQueuedRequests(FEmbeddingDerivedData&& EmbeddingData, FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastAllRequests");
	TUniqueFunction<void(FEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> EmbeddingCallback;
	TUniqueFunction<void(FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> CaptionCallback;
	TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> RecordCallback;

	{
		FWriteScopeLock WriteLock(IsBuildingDerivedDataLock);
		bIsBuildingDerivedData = false;
		EmbeddingCallback = NoLock_GetEmbeddingRequest();
		CaptionCallback = NoLock_GetCaptionRequests();
		RecordCallback = NoLock_GetRecordRequests();
	}

	// Count remaining consumers to know when we can move vs copy
	int32 RemainingConsumers = !!EmbeddingCallback + !!CaptionCallback + !!RecordCallback;

	if (EmbeddingCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastAllRequests::DispatchEmbedding");
		--RemainingConsumers;
		if (RemainingConsumers > 0)
		{
			EmbeddingCallback(FEmbeddingDerivedData(EmbeddingData), FString(ErrorMessage), Reason);
		}
		else
		{
			EmbeddingCallback(MoveTemp(EmbeddingData), MoveTemp(ErrorMessage), Reason);
		}
	}

	if (RecordCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastAllRequests::DispatchRecord");
		--RemainingConsumers;
		if (RemainingConsumers > 0)
		{
			RecordCallback(FEmbeddingDerivedData(EmbeddingData), FCaptionDerivedData(CaptionData), FString(ErrorMessage), Reason);
		}
		else
		{
			RecordCallback(MoveTemp(EmbeddingData), MoveTemp(CaptionData), MoveTemp(ErrorMessage), Reason);
		}
	}

	if (CaptionCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastAllRequests::DispatchCaption");
		CaptionCallback(MoveTemp(CaptionData), MoveTemp(ErrorMessage), Reason);
	}
}

TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> FSemanticSearchBuildCacheTask::NoLock_GetQuantizedRequests()
{
	TArray<TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>, TInlineAllocator<4>> Callbacks;
	bool bPreviousValue = true;
	HasActiveQuantizedRequest.compare_exchange_strong(bPreviousValue, false);
	if (bPreviousValue)
	{
		TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> Callback;
		while (QueuedQuantizedRequests.Dequeue(Callback))
		{
			Callbacks.Add(MoveTemp(Callback));
		}
	}

	if (Callbacks.IsEmpty())
	{
		return {};
	}

	return [Callbacks = MoveTemp(Callbacks)](FQuantizedEmbeddingDerivedData&& Data, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
	{
		for (int32 Index = 0; Index < Callbacks.Num(); ++Index)
		{
			if (Callbacks[Index])
			{
				if (Index < Callbacks.Num() - 1)
				{
					Callbacks[Index](FQuantizedEmbeddingDerivedData(Data), FString(ErrorMessage), Reason);
				}
				else
				{
					Callbacks[Index](MoveTemp(Data), MoveTemp(ErrorMessage), Reason);
				}
			}
		}
	};
}

void FSemanticSearchBuildCacheTask::BroadcastQueuedQuantizedRequest(FQuantizedEmbeddingDerivedData&& QuantizedData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastQuantizedRequest");
	TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> QuantizedCallback = NoLock_GetQuantizedRequests();

	if (QuantizedCallback)
	{
		QuantizedCallback(MoveTemp(QuantizedData), MoveTemp(ErrorMessage), Reason);
	}
}

TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> FSemanticSearchBuildCacheTask::NoLock_GetRecordRequests()
{
	TArray<TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>, TInlineAllocator<4>> Callbacks;
	bool bPreviousValue = true;
	HasActiveRecordRequest.compare_exchange_strong(bPreviousValue, false);
	if (bPreviousValue)
	{
		TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> Callback;
		while (QueuedRecordRequests.Dequeue(Callback))
		{
			Callbacks.Add(MoveTemp(Callback));
		}
	}

	if (Callbacks.IsEmpty())
	{
		return {};
	}

	return [Callbacks = MoveTemp(Callbacks)](FEmbeddingDerivedData&& EmbeddingData, FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason) mutable
	{
		for (int32 Index = 0; Index < Callbacks.Num(); ++Index)
		{
			if (Callbacks[Index])
			{
				if (Index < Callbacks.Num() - 1)
				{
					Callbacks[Index](FEmbeddingDerivedData(EmbeddingData), FCaptionDerivedData(CaptionData), FString(ErrorMessage), Reason);
				}
				else
				{
					Callbacks[Index](MoveTemp(EmbeddingData), MoveTemp(CaptionData), MoveTemp(ErrorMessage), Reason);
				}
			}
		}
	};
}

void FSemanticSearchBuildCacheTask::BroadcastQueuedRecordRequest(FEmbeddingDerivedData&& EmbeddingData, FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastRecordRequest");
	TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> RecordCallback;
	{
		FWriteScopeLock WriteLock(IsBuildingDerivedDataLock);
		RecordCallback = NoLock_GetRecordRequests();
	}

	if (RecordCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BroadcastRecordRequest::Dispatch");
		RecordCallback(MoveTemp(EmbeddingData), MoveTemp(CaptionData), MoveTemp(ErrorMessage), Reason);
	}
}

void FSemanticSearchBuildCacheTask::BuildQuantizedDerivedData(const FIoHash& CodebookHash, bool bBuildOnMiss)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SemanticSearch::BuildQuantizedDerivedData");

	// Get the float embedding (DDC hit in most cases, or triggers full BuildDerivedData if bBuildOnMiss)
	// SelfKeepAlive: BroadcastQueuedQuantizedRequest below can release the last strong ref (held by
	// BatchGetQuantizedRecordData's fallback BatchState) and self-destruct this FRequestOwner mid-callback
	// while an HTTP LaunchTask barrier on *this is still live up the stack.
	GetEmbeddingData([this, CodebookHash, SelfKeepAlive = AsShared()](FEmbeddingDerivedData&& EmbeddingData, FString&& Error, EAssetIndexFailureReason Reason)
		{
			if (!Error.IsEmpty() || EmbeddingData.Embedding.IsEmpty())
			{
				BroadcastQueuedQuantizedRequest(FQuantizedEmbeddingDerivedData(), MoveTemp(Error), Reason);
				return;
			}

			// Quantize on the consumer thread to avoid racing with index mutations (Need to revisit the pattern for that one)
			// SelfKeepAlive pins `this` until after the FRequestBarrier below exits — broadcasting can release the last
			// strong ref (held by batch state) and self-destruct this FRequestOwner mid-callback otherwise.
			FHybridSearchIndex::Get().QuantizeAsync(MoveTemp(EmbeddingData.Embedding),
				[this, CodebookHash, SelfKeepAlive = AsShared()](TArray<uint8>&& QuantizedCodes, bool bSuccess)
				{
					if (!bSuccess || QuantizedCodes.IsEmpty())
					{
						// Quantized index isn't trained — treat as retryable (once trained, the next attempt will succeed).
						BroadcastQueuedQuantizedRequest(FQuantizedEmbeddingDerivedData(), FString(TEXT("No trained quantized index")), EAssetIndexFailureReason::Provider);
						return;
					}

					FQuantizedEmbeddingDerivedData QuantizedData;
					QuantizedData.QuantizedEmbedding = MoveTemp(QuantizedCodes);

					// Write to DDC
					using namespace DerivedData;
					FCacheKey QuantizedKey = GenerateQuantizedCacheKey(AssetKey, CodebookHash);
					FCacheRecordBuilder RecordBuilder(QuantizedKey);
					TArray<uint8> Buffer;
					FMemoryWriter Writer(Buffer);
					SerializeQuantizedEmbedding(QuantizedData, Writer);
					RecordBuilder.AddValue(GetQuantizedEmbeddingDatachunkId(), MakeSharedBufferFromArray(MoveTemp(Buffer)));

					FCachePutRequest PutRequest{ DDCRequestName, RecordBuilder.Build(), GetCachePolicy() | ECachePolicy::SkipData };
					FRequestBarrier Barrier(*this);
					GetCache().Put({ PutRequest }, *this,
						[AssetPtr = Asset](FCachePutResponse&& Response)
						{
							if (Response.Status != DerivedData::EStatus::Ok)
							{
								UE_LOGF(LogSemanticSearch, Verbose, "Failed to store quantized codes in DDC for: %ls", *AssetPtr->GetObjectPathString());
							}
						});

					BroadcastQueuedQuantizedRequest(MoveTemp(QuantizedData), FString(), EAssetIndexFailureReason::None);
				});
		}, bBuildOnMiss);
}

// --- Batch DDC operations ---

/**
 * Shared state for a batch DDC Get() call.
 * Captures entries, callback, and request owner; destroyed when all responses complete.
 */
struct FBatchGetRecordState
{
	TArray<FBatchDDCEntry> Entries;
	TFunction<void(const FAssetData&, FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> Callback;
	bool bBuildOnMiss = true;
	// One owner per dispatched chunk — sharing a single owner across all chunks serialises begin/end
	// task bookkeeping inside the DDC request-owner lock and becomes the dominant cost on large batches.
	TArray<TUniquePtr<DerivedData::FRequestOwner>> ChunkOwners;

	explicit FBatchGetRecordState(
		TFunction<void(const FAssetData&, FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InCallback,
		bool bInBuildOnMiss)
		: Callback(MoveTemp(InCallback))
		, bBuildOnMiss(bInBuildOnMiss)
	{
	}
};

void FSemanticSearchBuildCacheTask::BatchGetRecordData(
	TArray<FBatchDDCEntry>&& Entries,
	TFunction<void(const FAssetData&, FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> OnAssetResult,
	bool bBuildOnMiss)
{
	using namespace DerivedData;

	if (Entries.IsEmpty())
	{
		return;
	}

	TSharedRef<FBatchGetRecordState> BatchState = MakeShared<FBatchGetRecordState>(MoveTemp(OnAssetResult), bBuildOnMiss);
	TArray<FCacheGetRequest> Requests;
	Requests.Reserve(Entries.Num());

	BatchState->Entries = MoveTemp(Entries);

	for (int32 i = 0; i < BatchState->Entries.Num(); ++i)
	{
		const FBatchDDCEntry& Entry = BatchState->Entries[i];

		FCacheGetRequest Request;
		Request.Name = Entry.Task->GetDDCRequestName();
		Request.Key = Entry.CacheKey;
		Request.Policy = GetCachePolicy();
		Request.UserData = static_cast<uint64>(i);
		Requests.Add(MoveTemp(Request));
	}

	auto OnResponse = [BatchState](FCacheGetResponse&& Response)
		{
			const int32 EntryIndex = static_cast<int32>(Response.UserData);
			FBatchDDCEntry& Entry = BatchState->Entries[EntryIndex];

			if (Response.Status == EStatus::Ok)
			{
				FEmbeddingDerivedData EmbData;
				const FValueWithId& EmbVal = Response.Record.GetValue(GetEmbeddingDatachunkId());
				if (EmbVal.HasData())
				{
					FSharedBuffer Buf = EmbVal.GetData().Decompress();
					FMemoryReaderView Reader(Buf.GetView(), true);
					SerializeEmbedding(EmbData, Reader);
				}

				FCaptionDerivedData CapData;
				const FValueWithId& CapVal = Response.Record.GetValue(GetCaptionDatachunkId());
				if (CapVal.HasData())
				{
					FSharedBuffer Buf = CapVal.GetData().Decompress();
					FMemoryReaderView Reader(Buf.GetView(), true);
					SerializeCaption(CapData, Reader);
				}

				BatchState->Callback(Entry.Asset, MoveTemp(EmbData), MoveTemp(CapData), FString(), EAssetIndexFailureReason::None);
			}
			else if (Response.Status != EStatus::Canceled)
			{
				// The chunk owner is separate from the per-task FRequestOwner cancelled by CancelAllTasks,
				// so a miss can still land after cancel. so Early exit before dispatching a build.
				if (Entry.Task && Entry.Task->IsCanceled())
				{
					BatchState->Callback(Entry.Asset, FEmbeddingDerivedData(), FCaptionDerivedData(), FString(TEXT("Cancelled")), EAssetIndexFailureReason::None);
				}
				// We already know DDC missed from the batch Get, so skip the redundant lookup and go straight to a build.
				else if (BatchState->bBuildOnMiss && Entry.Task)
				{
					Entry.Task->BuildRecordData(
						[Asset = FAssetData(Entry.Asset), Callback = BatchState->Callback](
							FEmbeddingDerivedData&& EmbData,
							FCaptionDerivedData&& CapData,
							FString&& Error,
							EAssetIndexFailureReason Reason) mutable
						{
							Callback(Asset, MoveTemp(EmbData), MoveTemp(CapData), MoveTemp(Error), Reason);
						});
				}
				else
				{
					// Cache miss without a build fallback — not a failure to mark.
					BatchState->Callback(Entry.Asset, FEmbeddingDerivedData(), FCaptionDerivedData(), FString(TEXT("Not cached")), EAssetIndexFailureReason::None);
				}
			}
			else
			{
				// Cancel — not a failure to mark.
				BatchState->Callback(Entry.Asset, FEmbeddingDerivedData(), FCaptionDerivedData(), FString(TEXT("Cancelled")), EAssetIndexFailureReason::None);
			}
		};

	// Split into chunks of at most MaxDDCBatchSize to avoid Zen server timeouts on large batches.
	const int32 NumRequests = Requests.Num();
	BatchState->ChunkOwners.Reserve(FMath::DivideAndRoundUp(NumRequests, MaxDDCBatchSize));
	for (int32 ChunkStart = 0; ChunkStart < NumRequests; ChunkStart += MaxDDCBatchSize)
	{
		const int32 ChunkCount = FMath::Min(MaxDDCBatchSize, NumRequests - ChunkStart);
		FRequestOwner& ChunkOwner = *BatchState->ChunkOwners.Add_GetRef(MakeUnique<FRequestOwner>(EPriority::Low));
		ChunkOwner.KeepAlive();
		FRequestBarrier Barrier(ChunkOwner);
		GetCache().Get(MakeArrayView(Requests.GetData() + ChunkStart, ChunkCount), ChunkOwner, OnResponse);
	}
}

/**
 * Shared state for a quantized batch DDC operation (requires two parallel batch Get calls).
 */
struct FBatchGetQuantizedState
{
	struct FEntry
	{
		FAssetData Asset;
		TSharedPtr<FSemanticSearchBuildCacheTask> Task;
		DerivedData::FCacheKey BaseKey;
		DerivedData::FCacheKey QuantizedKey;
	};

	TArray<FEntry> Entries;
	TFunction<void(const FAssetData&, FCaptionDerivedData&&, FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> Callback;
	bool bBuildOnMiss = true;
	FIoHash CodebookHash;
	// One owner per dispatched chunk per stream — sharing a single owner across all chunks serialises
	// begin/end task bookkeeping inside the DDC request-owner lock and becomes the dominant cost on
	// large batches.
	TArray<TUniquePtr<DerivedData::FRequestOwner>> BaseChunkOwners;
	TArray<TUniquePtr<DerivedData::FRequestOwner>> QuantizedChunkOwners;

	// Per-asset tracking: atomic counter + partial results
	struct FPerAssetTracker
	{
		FCriticalSection Lock;
		FCaptionDerivedData CaptionResult;
		FQuantizedEmbeddingDerivedData QuantizedResult;
		FString Error;
		EAssetIndexFailureReason Reason = EAssetIndexFailureReason::None;
		std::atomic<int32> CompletedCount{0};
	};
	TArray<TUniquePtr<FPerAssetTracker>> Trackers;

	explicit FBatchGetQuantizedState(
		TFunction<void(const FAssetData&, FCaptionDerivedData&&, FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InCallback,
		bool bInBuildOnMiss, const FIoHash& InCodebookHash)
		: Callback(MoveTemp(InCallback))
		, bBuildOnMiss(bInBuildOnMiss)
		, CodebookHash(InCodebookHash)
	{
	}
};

void FSemanticSearchBuildCacheTask::BatchGetQuantizedRecordData(
	TArray<FBatchDDCEntry>&& Entries,
	const FIoHash& CodebookHash,
	TFunction<void(const FAssetData&, FCaptionDerivedData&&, FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> OnAssetResult,
	bool bBuildOnMiss)
{
	using namespace DerivedData;

	if (Entries.IsEmpty())
	{
		return;
	}

	TSharedRef<FBatchGetQuantizedState> BatchState = MakeShared<FBatchGetQuantizedState>(MoveTemp(OnAssetResult), bBuildOnMiss, CodebookHash);
	TArray<FCacheGetChunkRequest> BaseRequests;
	TArray<FCacheGetRequest> QuantizedRequests;
	BaseRequests.Reserve(Entries.Num());
	QuantizedRequests.Reserve(Entries.Num());
	BatchState->Entries.Reserve(Entries.Num());

	for (int32 i = 0; i < Entries.Num(); ++i)
	{
		FBatchDDCEntry& Entry = Entries[i];
		FCacheKey QuantizedKey = GenerateQuantizedCacheKey(Entry.CacheKey, CodebookHash);

		const uint64 EntryIdx = static_cast<uint64>(BatchState->Entries.Num());

		// Base caption chunk request (skips the embedding payload we don't need here)
		{
			FCacheGetChunkRequest Request;
			Request.Name = Entry.Task->GetDDCRequestName();
			Request.Key = Entry.CacheKey;
			Request.Id = GetCaptionDatachunkId();
			Request.Policy = GetCachePolicy();
			Request.UserData = EntryIdx;
			BaseRequests.Add(MoveTemp(Request));
		}

		// Quantized record request (for quantized codes)
		{
			FCacheGetRequest Request;
			Request.Name = Entry.Task->GetDDCRequestName();
			Request.Key = QuantizedKey;
			Request.Policy = GetCachePolicy();
			Request.UserData = EntryIdx;
			QuantizedRequests.Add(MoveTemp(Request));
		}

		BatchState->Entries.Add({ MoveTemp(Entry.Asset), MoveTemp(Entry.Task), MoveTemp(Entry.CacheKey), MoveTemp(QuantizedKey) });
		BatchState->Trackers.Add(MakeUnique<FBatchGetQuantizedState::FPerAssetTracker>());
	}

	auto TryFireCallback = [BatchState](int32 EntryIndex)
	{
		FBatchGetQuantizedState::FPerAssetTracker& Tracker = *BatchState->Trackers[EntryIndex];
		if (Tracker.CompletedCount.fetch_add(1, std::memory_order_acq_rel) == 1)
		{

			// Fix this.
			FScopeLock ScopeLock(&Tracker.Lock);
			BatchState->Callback(BatchState->Entries[EntryIndex].Asset, MoveTemp(Tracker.CaptionResult), MoveTemp(Tracker.QuantizedResult), MoveTemp(Tracker.Error), Tracker.Reason);
		}
	};

	// Batch 1: Caption chunk only (the embedding chunk in the base record is not needed here)
	auto OnBaseResponse = [BatchState, TryFireCallback](FCacheGetChunkResponse&& Response)
		{
			const int32 EntryIndex = static_cast<int32>(Response.UserData);
			FBatchGetQuantizedState::FPerAssetTracker& Tracker = *BatchState->Trackers[EntryIndex];
			FBatchGetQuantizedState::FEntry& Entry = BatchState->Entries[EntryIndex];

			if (Response.Status == EStatus::Ok && Response.RawData.GetSize() > 0)
			{
				FCaptionDerivedData CapData;
				FMemoryReaderView Reader(Response.RawData.GetView(), true);
				SerializeCaption(CapData, Reader);
				Response.RawData.Reset();

				FScopeLock ScopeLock(&Tracker.Lock);
				Tracker.CaptionResult = MoveTemp(CapData);
			}
			else if (Response.Status != EStatus::Canceled && BatchState->bBuildOnMiss && Entry.Task && !Entry.Task->IsCanceled())
			{
				// We already know DDC missed from the batch Get, so skip the redundant lookup and go straight to a build.
				Entry.Task->BuildCaptionData(
					[BatchState, EntryIndex, TryFireCallback](
						FCaptionDerivedData&& CapData, FString&& Error, EAssetIndexFailureReason Reason) mutable
					{
						FBatchGetQuantizedState::FPerAssetTracker& FallbackTracker = *BatchState->Trackers[EntryIndex];
						{
							FScopeLock ScopeLock(&FallbackTracker.Lock);
							FallbackTracker.CaptionResult = MoveTemp(CapData);
							if (!Error.IsEmpty() && FallbackTracker.Error.IsEmpty())
							{
								FallbackTracker.Error = MoveTemp(Error);
								FallbackTracker.Reason = Reason;
							}
						}
						TryFireCallback(EntryIndex);
					});
				return; // TryFireCallback will be invoked by the nested callback
			}
			else if (Response.Status != EStatus::Canceled)
			{
				// Cache miss without a build fallback — not a failure to mark.
				FScopeLock ScopeLock(&Tracker.Lock);
				if (Tracker.Error.IsEmpty())
				{
					Tracker.Error = TEXT("Caption cache miss");
				}
			}
			else
			{
				// Cancel — not a failure to mark.
				FScopeLock ScopeLock(&Tracker.Lock);
				Tracker.Error = TEXT("Cancelled");
			}

			TryFireCallback(EntryIndex);
		};

	// Batch 2: Quantized records (quantized codes)
	auto OnQuantizedResponse = [BatchState, TryFireCallback](FCacheGetResponse&& Response)
		{
			const int32 EntryIndex = static_cast<int32>(Response.UserData);
			FBatchGetQuantizedState::FPerAssetTracker& Tracker = *BatchState->Trackers[EntryIndex];
			FBatchGetQuantizedState::FEntry& Entry = BatchState->Entries[EntryIndex];

			if (Response.Status == EStatus::Ok)
			{
				const FValueWithId& QVal = Response.Record.GetValue(GetQuantizedEmbeddingDatachunkId());
				if (QVal.HasData())
				{
					FQuantizedEmbeddingDerivedData QData;
					FSharedBuffer Buf = QVal.GetData().Decompress();
					FMemoryReaderView Reader(Buf.GetView(), true);
					SerializeQuantizedEmbedding(QData, Reader);

					FScopeLock ScopeLock(&Tracker.Lock);
					Tracker.QuantizedResult = MoveTemp(QData);
				}
			}
			else if (Response.Status != EStatus::Canceled && BatchState->bBuildOnMiss && Entry.Task && !Entry.Task->IsCanceled())
			{
				// We already know DDC missed from the batch Get, so skip the redundant lookup and go straight to a build.
				Entry.Task->BuildQuantizedEmbeddingData(BatchState->CodebookHash,
					[BatchState, EntryIndex, TryFireCallback](
						FQuantizedEmbeddingDerivedData&& QData, FString&& Error, EAssetIndexFailureReason Reason) mutable
					{
						FBatchGetQuantizedState::FPerAssetTracker& FallbackTracker = *BatchState->Trackers[EntryIndex];
						{
							FScopeLock ScopeLock(&FallbackTracker.Lock);
							FallbackTracker.QuantizedResult = MoveTemp(QData);
							if (!Error.IsEmpty() && FallbackTracker.Error.IsEmpty())
							{
								FallbackTracker.Error = MoveTemp(Error);
								FallbackTracker.Reason = Reason;
							}
						}
						TryFireCallback(EntryIndex);
					});
				return; // TryFireCallback called by the nested callback
			}
			else
			{
				// Cache miss / cancel — not a failure to mark.
				FScopeLock ScopeLock(&Tracker.Lock);
				if (Tracker.Error.IsEmpty())
				{
					Tracker.Error = Response.Status == EStatus::Canceled ? TEXT("Cancelled") : TEXT("Not cached");
				}
			}

			TryFireCallback(EntryIndex);
		};

	
	// Split into chunks of at most MaxDDCBatchSize to avoid long timeouts on large batches.
	const int32 NumBaseRequests = BaseRequests.Num();
	check(QuantizedRequests.Num() == BaseRequests.Num());
	const int32 NumChunks = FMath::DivideAndRoundUp(NumBaseRequests, MaxDDCBatchSize);
	BatchState->BaseChunkOwners.Reserve(NumChunks);
	BatchState->QuantizedChunkOwners.Reserve(NumChunks);
	for (int32 ChunkStart = 0; ChunkStart < NumBaseRequests; ChunkStart += MaxDDCBatchSize)
	{
		const int32 ChunkCount = FMath::Min(MaxDDCBatchSize, NumBaseRequests - ChunkStart);

		FRequestOwner& BaseChunkOwner = *BatchState->BaseChunkOwners.Add_GetRef(MakeUnique<FRequestOwner>(EPriority::Low));
		BaseChunkOwner.KeepAlive();
		FRequestOwner& QuantizedChunkOwner = *BatchState->QuantizedChunkOwners.Add_GetRef(MakeUnique<FRequestOwner>(EPriority::Low));
		QuantizedChunkOwner.KeepAlive();

		FRequestBarrier BaseBarrier(BaseChunkOwner);
		FRequestBarrier QuantizedBarrier(QuantizedChunkOwner);
		GetCache().GetChunks(MakeArrayView(BaseRequests.GetData() + ChunkStart, ChunkCount), BaseChunkOwner, OnBaseResponse);
		GetCache().Get(MakeArrayView(QuantizedRequests.GetData() + ChunkStart, ChunkCount), QuantizedChunkOwner, OnQuantizedResponse);
	}

}

}
