// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/SharedString.h"
#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataCachePolicy.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataValueId.h"
#include "HAL/CriticalSection.h"
#include "HybridSearchIndex.h"
#include "IO/IoHash.h"
#include "Templates/SharedPointer.h"
#include <atomic>

namespace UE::SemanticSearch
{

class IAssetProcessor;

namespace Private
{
/**
 * We use a guid in the cpp file to track the version of the data. Make sure to change the guid if those are modified (this will invalidate all the cached data)
 */

struct FEmbeddingDerivedData
{
	TArray<float> Embedding;
};

struct FQuantizedEmbeddingDerivedData
{
	TArray<uint8> QuantizedEmbedding;
};

struct FCaptionDerivedData
{
	FString Caption;
	TArray<FString> Keywords;
};

// DDC value IDs, serialization, and cache policy helpers (defined in SemanticSearchDerivedDataBuilder.cpp)
DerivedData::FValueId GetEmbeddingDatachunkId();
DerivedData::FValueId GetCaptionDatachunkId();
DerivedData::FValueId GetQuantizedEmbeddingDatachunkId();
void SerializeEmbedding(FEmbeddingDerivedData& Embedding, FArchive& Archive);
void SerializeQuantizedEmbedding(FQuantizedEmbeddingDerivedData& QuantizedEmbedding, FArchive& Archive);
void SerializeCaption(FCaptionDerivedData& Caption, FArchive& Archive);
DerivedData::ECachePolicy GetCachePolicy();

class FSemanticSearchBuildCacheTask; // forward decl for FBatchDDCEntry

/** Entry for batch DDC operations. Populated by AssetProcessorManager, consumed by static batch methods. */
struct FBatchDDCEntry
{
	FAssetData Asset;
	TSharedPtr<FSemanticSearchBuildCacheTask> Task;
	DerivedData::FCacheKey CacheKey;
};

class FSemanticSearchBuildCacheTask
	: public DerivedData::FRequestOwner
	, public TSharedFromThis<FSemanticSearchBuildCacheTask>
{
public:
	// InKey must not be empty — passing an empty key will crash. Use GenerateCacheKey and validate before calling.
	static TSharedPtr<FSemanticSearchBuildCacheTask> Create(const FAssetData& InAssetData, const TSharedRef<IAssetProcessor>& InAssetProcessor, DerivedData::FCacheKey&& InKey);
	static TSharedPtr<FSemanticSearchBuildCacheTask> Create(FAssetData&& InAssetData, const TSharedRef<IAssetProcessor>& InAssetProcessor, DerivedData::FCacheKey&& InKey);

	FSemanticSearchBuildCacheTask(FAssetData&& InAssetData, const TSharedRef<IAssetProcessor>& InAssetProcessor, DerivedData::FCacheKey&& InAssetKey);

	void GetEmbeddingData(TUniqueFunction<void (FEmbeddingDerivedData&& EmbeddingData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)>&& InOnDataAvailable, bool bBuildOnMiss = true);
	void GetCaptionData(TUniqueFunction<void (FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason)>&& InOnDataAvailable);

	/** Retrieve quantized embedding data from DDC. On miss, builds by fetching the float embedding and quantizing. */
	void GetQuantizedEmbeddingData(const FIoHash& CodebookHash,
		TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable,
		bool bBuildOnMiss = true);

	/** Retrieve both embedding and caption in a single DDC Get() call instead of 2x GetChunks(). */
	void GetRecordData(
		TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable,
		bool bBuildOnMiss = true);

	void CacheData();

	const DerivedData::FCacheKey& GetAssetKey() const { return AssetKey; }
	const FSharedString& GetDDCRequestName() const { return DDCRequestName; }

	static DerivedData::FCacheKey GenerateCacheKey(const FAssetData& InAssetData, const TSharedRef<IAssetProcessor>& InAssetProcessor);
	static DerivedData::FCacheKey GenerateQuantizedCacheKey(const DerivedData::FCacheKey& BaseCacheKey, const FIoHash& CodebookHash);
	static DerivedData::FCacheBucket GetCacheKeyBucket(const TSharedRef<IAssetProcessor>& InAssetProcessor);

	/** Batch-fetch embedding + caption from DDC for multiple assets. Falls back to per-asset GetRecordData on miss. */
	static void BatchGetRecordData(
		TArray<FBatchDDCEntry>&& Entries,
		TFunction<void(const FAssetData&, FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> OnAssetResult,
		bool bBuildOnMiss = true);

	/** Batch-fetch caption + quantized codes from DDC for multiple assets (two parallel batch Get calls). */
	static void BatchGetQuantizedRecordData(
		TArray<FBatchDDCEntry>&& Entries,
		const FIoHash& CodebookHash,
		TFunction<void(const FAssetData&, FCaptionDerivedData&&, FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> OnAssetResult,
		bool bBuildOnMiss = true);

private:

	void BuildDerivedData();
	void BuildQuantizedDerivedData(const FIoHash& CodebookHash, bool bBuildOnMiss);

	/** Like GetCaptionData, but skips the DDC lookup. Use when a prior batch Get has already confirmed a miss. */
	void BuildCaptionData(TUniqueFunction<void(FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable);

	/** Like GetRecordData, but skips the DDC lookup. Use when a prior batch Get has already confirmed a miss. */
	void BuildRecordData(TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable);

	/** Like GetQuantizedEmbeddingData, but skips the DDC lookup. Use when a prior batch Get has already confirmed a miss. */
	void BuildQuantizedEmbeddingData(const FIoHash& CodebookHash, TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable);
	void BroadcastQueuedRequests(FEmbeddingDerivedData&& EmbeddingData, FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason);
	void BroadcastQueuedCaptionRequest(FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason);
	void BroadcastQueuedEmbeddingRequest(FEmbeddingDerivedData&& EmbeddingData, FString&& ErrorMessage, EAssetIndexFailureReason Reason);
	void BroadcastQueuedQuantizedRequest(FQuantizedEmbeddingDerivedData&& QuantizedData, FString&& ErrorMessage, EAssetIndexFailureReason Reason);
	void BroadcastQueuedRecordRequest(FEmbeddingDerivedData&& EmbeddingData, FCaptionDerivedData&& CaptionData, FString&& ErrorMessage, EAssetIndexFailureReason Reason);

	TUniqueFunction<void(FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> NoLock_GetCaptionRequests();
	TUniqueFunction<void(FEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> NoLock_GetEmbeddingRequest();
	TUniqueFunction<void(FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)> NoLock_GetQuantizedRequests();
	TUniqueFunction<void(FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)> NoLock_GetRecordRequests();


	TSharedRef<const FAssetData> Asset;
	TSharedRef<IAssetProcessor> AssetProcessor;
	DerivedData::FCacheKey AssetKey;
	FSharedString DDCRequestName;

	// Acs has a lock for the IsBuildingDerivedData flag but also avoid race condition with the queued Requests
	FRWLock IsBuildingDerivedDataLock;
	bool bIsBuildingDerivedData = false;
	TQueue<TUniqueFunction<void (FEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>, EQueueMode::Mpsc> QueuedEmbeddingRequests;
	TQueue<TUniqueFunction<void (FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>, EQueueMode::Mpsc> QueuedCaptionRequests;
	TQueue<TUniqueFunction<void (FQuantizedEmbeddingDerivedData&&, FString&&, EAssetIndexFailureReason)>, EQueueMode::Mpsc> QueuedQuantizedRequests;
	TQueue<TUniqueFunction<void (FEmbeddingDerivedData&&, FCaptionDerivedData&&, FString&&, EAssetIndexFailureReason)>, EQueueMode::Mpsc> QueuedRecordRequests;
	std::atomic<bool> HasActiveEmbeddingRequest = false;
	std::atomic<bool> HasActiveCaptionRequest = false;
	std::atomic<bool> HasActiveQuantizedRequest = false;
	std::atomic<bool> HasActiveRecordRequest = false;
};
}
}
