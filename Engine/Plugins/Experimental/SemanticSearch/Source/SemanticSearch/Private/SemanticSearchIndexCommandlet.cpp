// Copyright Epic Games, Inc. All Rights Reserved.

#include "SemanticSearchIndexCommandlet.h"

#include "AssetProcessors/AssetProcessorUtils.h"
#include "HybridSearchIndex.h"
#include "ISemanticSearchModule.h"
#include "SemanticSearchModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/ConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "HttpManager.h"
#include "HttpModule.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Containers/Ticker.h"

#include <atomic>

DEFINE_LOG_CATEGORY_STATIC(LogSemanticSearchCommandlet, Log, All);

USemanticSearchIndexCommandlet::USemanticSearchIndexCommandlet()
{
	IsClient = false;
	IsEditor = true;
	IsServer = false;
	LogToConsole = true;
}

int32 USemanticSearchIndexCommandlet::Main(const FString& Params)
{
	using namespace UE::SemanticSearch;

	LogSemanticSearch.SetVerbosity(ELogVerbosity::Verbose);
	LogSemanticSearchCommandlet.SetVerbosity(ELogVerbosity::Verbose);

	// Parse timeout (default 4 hours)
	int32 TimeoutSeconds = 14400;
	FParse::Value(*Params, TEXT("-Timeout="), TimeoutSeconds);

	UE_LOG(LogSemanticSearchCommandlet, Log, TEXT("SemanticSearchIndex commandlet starting (timeout=%ds)"), TimeoutSeconds);

	// 1. Wait for asset registry to finish scanning
	UE_LOG(LogSemanticSearchCommandlet, Log, TEXT("Waiting for asset registry..."));
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.SearchAllAssets(/*bSynchronousSearch=*/true);
	UE_LOG(LogSemanticSearchCommandlet, Log, TEXT("Asset registry loaded"));

	// 2. Kick off full indexing
	ISemanticSearchModule& Module = ISemanticSearchModule::Get();
	Module.IndexAllAssets(/*bForceBuild=*/true);

	// 3. Pump engine systems until indexing completes or timeout
	const double StartTime = FPlatformTime::Seconds();
	const float DeltaTime = 0.05f;

	UE_LOG(LogSemanticSearchCommandlet, Log, TEXT("Indexing in progress, pumping engine systems..."));

	while (Module.IsIndexingInProgress())
	{
		const double Elapsed = FPlatformTime::Seconds() - StartTime;
		if (Elapsed > static_cast<double>(TimeoutSeconds))
		{
			UE_LOG(LogSemanticSearchCommandlet, Error, TEXT("Indexing timed out after %d seconds"), TimeoutSeconds);
			Module.CancelIndexing();
			FHybridSearchIndex::Get().StopCommandQueue();
			return 1;
		}

		// Tick HTTP so embedding requests complete
		FHttpModule::Get().GetHttpManager().Tick(DeltaTime);

		// Tick async tasks (DDC, etc.)
		FTSTicker::GetCoreTicker().Tick(DeltaTime);

		// Process game-thread tasks dispatched via AsyncTask(ENamedThreads::GameThread, ...)
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		FPlatformProcess::Sleep(DeltaTime);
	}

	const double ElapsedTotal = FPlatformTime::Seconds() - StartTime;

	// Log final stats
	FSemanticSearchIndexStats Stats = FHybridSearchIndex::Get().GetCachedIndexStats();
	UE_LOG(LogSemanticSearchCommandlet, Log, TEXT("Indexing complete in %.1fs (vector: %d, BM25: %d, failed: %d)"),
		ElapsedTotal, Stats.VectorCount, Stats.BM25Count, Stats.FailedCount);

	// 4. Drain the command queue and shut down
	FHybridSearchIndex::Get().StopCommandQueue();
	return 0;
}
