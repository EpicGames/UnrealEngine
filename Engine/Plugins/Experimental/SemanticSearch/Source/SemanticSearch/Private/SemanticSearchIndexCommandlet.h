// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "SemanticSearchIndexCommandlet.generated.h"

/**
 * Commandlet that indexes all project assets for semantic search and saves the
 * index to the shared project Config directory for Perforce distribution.
 *
 * Usage: UnrealEditor-Cmd <Project> -run=SemanticSearchIndex [-Timeout=<seconds>]
 *
 * The generated index is written to <ProjectDir>/Config/SemanticSearch/SearchIndex.bin
 * and is intended to be submitted to source control by a BuildGraph Submit step.
 */
UCLASS()
class USemanticSearchIndexCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	USemanticSearchIndexCommandlet();
	virtual int32 Main(const FString& Params) override;
};
