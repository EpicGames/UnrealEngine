// Copyright Epic Games, Inc. All Rights Reserved.

#include "SemanticSearchModule.h"

#include "AssetProcessorManager.h"
#include "AssetProcessors/AssetProcessorUtils.h"
#include "HybridSearchIndex.h"
#include "TextIndex/BM25Index.h"
#include "Interfaces/IQuantizedVectorIndex.h"
#include "VectorIndex/VectorIndexFactory.h"

#include "Modules/ModuleManager.h"
#include "EmbeddingProviders/OpenAIEmbeddingProvider.h"
#include "Async/Async.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Settings/SemanticSearchSettings.h"
#include "HAL/ConsoleManager.h"
#include "HttpModule.h"
#include "Misc/Paths.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY(LogSemanticSearch);

struct FHttpSettingsSnapshot
{
	int32 MaxHostConnections    = -1;
	int32 MaxConcurrentRequests = -1;
};

static FHttpSettingsSnapshot GOriginalHttpSettings;
static bool GHttpSettingsOverridden = false;

static int32 GetCVarInt(const TCHAR* Name)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(Name);
	return CVar ? CVar->GetInt() : -1;
}

static void SetCVarInt(const TCHAR* Name, int32 Value)
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(Name))
	{
		CVar->Set(Value, ECVF_SetByCode);
		UE_LOG(LogSemanticSearch, Log, TEXT("  %s = %d"), Name, Value);
	}
	else
	{
		UE_LOG(LogSemanticSearch, Log, TEXT("  CVar %s not found"), Name);
	}
}

static void ResetHttpOverrides()
{
	if (!GHttpSettingsOverridden)
	{
		return;
	}

	SetCVarInt(TEXT("HTTP.Curl.MaxHostConnections"), GOriginalHttpSettings.MaxHostConnections);
	SetCVarInt(TEXT("http.MaxConcurrentRequests"),   GOriginalHttpSettings.MaxConcurrentRequests);

	GHttpSettingsOverridden = false;
}

static void ApplyHttpOverrides()
{
	if (GHttpSettingsOverridden)
	{
		return;
	}

	GOriginalHttpSettings.MaxHostConnections    = GetCVarInt(TEXT("HTTP.Curl.MaxHostConnections"));
	GOriginalHttpSettings.MaxConcurrentRequests = GetCVarInt(TEXT("http.MaxConcurrentRequests"));

	SetCVarInt(TEXT("HTTP.Curl.MaxHostConnections"), 0);
	SetCVarInt(TEXT("http.MaxConcurrentRequests"),   9999);

	GHttpSettingsOverridden = true;
}

/** Construct the embedding provider that matches the current settings. */
static TUniquePtr<UE::SemanticSearch::IEmbeddingProvider> CreateConfiguredEmbeddingProvider()
{
	using namespace UE::SemanticSearch;
	using namespace UE::SemanticSearch::Private;

	const ESemanticSearchEmbeddingProvider ProviderType = USemanticSearchSettings::Get()
		? USemanticSearchSettings::Get()->Provider
		: ESemanticSearchEmbeddingProvider::OpenAI;
	switch (ProviderType)
	{
		case ESemanticSearchEmbeddingProvider::OpenAI:
		default:
			return MakeUnique<FOpenAIEmbeddingProvider>();
	}
}

void USemanticSearchSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (!PropertyChangedEvent.MemberProperty)
	{
		UE::SemanticSearch::FAssetProcessorManager::Get().InvalidateSupportedAssetCount();
		UE::SemanticSearch::ISemanticSearchModule::Get().RegisterEmbeddingProvider(CreateConfiguredEmbeddingProvider(), true);
		return;
	}

	const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(USemanticSearchSettings, IndexedPaths))
	{
		UE::SemanticSearch::FAssetProcessorManager::Get().InvalidateSupportedAssetCount();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(USemanticSearchSettings, Provider)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(USemanticSearchSettings, EmbeddingModel)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(USemanticSearchSettings, EmbeddingDimension))
	{
		// Re-register provider and invalidate index
		UE::SemanticSearch::ISemanticSearchModule::Get().RegisterEmbeddingProvider(CreateConfiguredEmbeddingProvider(), true);
		UE_LOGF(LogSemanticSearch, Log, "Re-registered embedding provider after settings change (%ls).",
			PropertyName == GET_MEMBER_NAME_CHECKED(USemanticSearchSettings, Provider)
				? TEXT("provider switched")
				: TEXT("config field edited"));
	}
	else if (PropertyChangedEvent.MemberProperty->GetMetaData(TEXT("Category")) == TEXT("Provider")
		|| PropertyChangedEvent.MemberProperty->GetMetaData(TEXT("Category")) == TEXT("HTTP"))
	{
		// Re-register provider but keep current index
		UE::SemanticSearch::ISemanticSearchModule::Get().RegisterEmbeddingProvider(CreateConfiguredEmbeddingProvider(), false);
		UE_LOGF(LogSemanticSearch, Log, "Re-registered embedding provider after settings change (%ls).",
			PropertyName == GET_MEMBER_NAME_CHECKED(USemanticSearchSettings, Provider)
				? TEXT("provider switched")
				: TEXT("config field edited"));
	}
}

bool UE::SemanticSearch::ISemanticSearchModule::IsInIndexedFolder(FStringView PackagePath)
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	if (!Settings) { return false; }
	for (const FDirectoryPath& Dir : Settings->IndexedPaths)
	{
		if (PackagePath.StartsWith(Dir.Path))
		{
			return true;
		}
	}
	return false;
}

bool UE::SemanticSearch::ISemanticSearchModule::IsInIndexedFolder(const FAssetData& AssetData)
{
	return IsInIndexedFolder(AssetData.PackageName.ToString());
}

namespace UE::SemanticSearch::Private
{
IMPLEMENT_MODULE(FSemanticSearchModule, SemanticSearch)

static FIoHash GetPackageSavedHash(const FAssetData& AssetData)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	TOptional<FAssetPackageData> PkgData = AssetRegistry.GetAssetPackageDataCopy(AssetData.PackageName);
	return PkgData ? PkgData->GetPackageSavedHash() : FIoHash();
}

FString FSemanticSearchModule::GetSavedIndexPath()
{
	return FPaths::ProjectSavedDir() / TEXT("SemanticSearch") / TEXT("SearchIndex.bin");
}

FString FSemanticSearchModule::GetPreProcessorFailedPath()
{
	return FPaths::ProjectSavedDir() / TEXT("SemanticSearch") / TEXT("PreProcessorFailed.bin");
}

bool FSemanticSearchModule::IsIndexingInProgress() const
{
	return bIndexingInProgress.load(std::memory_order_acquire);
}

FSemanticSearchModule& FSemanticSearchModule::Get()
{
	static const FName ModuleName(TEXT("SemanticSearch"));
	return FModuleManager::Get().GetModuleChecked<FSemanticSearchModule>(ModuleName);
}

/**
 * When true, FSemanticSearchModule::StartupModule() skips loading the index, starting the
 * consumer queue, and binding asset-registry delegates. The owning module (or a plugin that
 * installs a custom embedding provider) must then call ISemanticSearchModule::Initialize()
 * explicitly to finalize startup. Set via DefaultEngine.ini [ConsoleVariables].
 */
static bool GSemanticSearchDelayInitialization = false;
static FAutoConsoleVariableRef CVarSemanticSearchDelayInitialization(
	TEXT("SemanticSearch.DelayInitialization"),
	GSemanticSearchDelayInitialization,
	TEXT("If true, defers loading the saved index, starting the consumer queue, and binding asset-registry delegates until ISemanticSearchModule::Initialize() is called explicitly."),
	ECVF_ReadOnly);

void FSemanticSearchModule::StartupModule()
{
	UE_LOGF(LogSemanticSearch, Verbose, "SemanticSearch module starting up");
	TUniquePtr<IEmbeddingProvider> Provider = CreateConfiguredEmbeddingProvider();
	RegisteredEmbeddingProvider = MakeShareable<IEmbeddingProvider>(Provider.Release());
	RegisterDefaultAssetProcessors();

	if (!GSemanticSearchDelayInitialization)
	{
		Initialize();
	}
	else
	{
		UE_LOGF(LogSemanticSearch, Log, "SemanticSearch.DelayInitialization=1; waiting for explicit Initialize() call");
	}
}

void FSemanticSearchModule::Initialize()
{
	check(IsInGameThread());

	if (bIsInitialized)
	{
		UE_LOGF(LogSemanticSearch, Verbose, "Initialize: already initialized; ignoring");
		return;
	}

	// Load saved index before starting the command queue (LoadFromFile is not thread-safe with consumer)
	FHybridSearchIndex& Index = FHybridSearchIndex::Get();
	const FString IndexPath = GetSavedIndexPath();
	const int32 ProviderDimension = RegisteredEmbeddingProvider->GetEmbeddingDimension();
	if (Index.LoadFromFile(IndexPath, LoadedPackageHashes))
	{
		FSemanticSearchIndexStats Stats = Index.GetCachedIndexStats();
		UE_LOGF(LogSemanticSearch, Log, "Loaded saved index from %ls (vector: %d, BM25: %d, hashes: %d)",
			*IndexPath, Stats.VectorCount, Stats.BM25Count, LoadedPackageHashes.Num());

		// A custom provider may have been registered before Initialize() in delayed-init mode. The
		// saved index on disk was sized for the previous provider's dimension; if the dimensions
		// disagree the loaded vectors are unusable. Rebuild as Flat with the new dimension before
		// the command queue starts so no incompatible embeddings can be added.
		if (ProviderDimension > 0 && Stats.Dimension > 0 && Stats.Dimension != ProviderDimension)
		{
			UE_LOGF(LogSemanticSearch, Log,
				"Saved index dimension %d differs from registered provider dimension %d; rebuilding as Flat",
				Stats.Dimension, ProviderDimension);
			RebuildVectorIndexAsFlat(ProviderDimension);
			// Hashes pair with IDs in the discarded index; reusing them would mark stale assets fresh.
			LoadedPackageHashes.Empty();
		}
	}
	else
	{
		Index.EnsureInitialized(ProviderDimension);
	}

	// Sidecar: load permanent pre-processor failures. Missing file is normal on first run.
	Index.LoadPreProcessorFailedFromFile(GetPreProcessorFailedPath());

	Index.StartCommandQueue();

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	// Todo Speed UP
	//AssetAddedDelegateHandle = AssetRegistry.OnAssetAdded().AddRaw(this, &FSemanticSearchModule::OnAssetAdded);
	PackageSavedDelegateHandle = UPackage::PackageSavedWithContextEvent.AddRaw(this, &FSemanticSearchModule::OnPackageSaved);
	AssetRenamedDelegateHandle = AssetRegistry.OnAssetRenamed().AddRaw(this, &FSemanticSearchModule::OnAssetRenamed);
	AssetRemovedDelegateHandle = AssetRegistry.OnAssetRemoved().AddRaw(this, &FSemanticSearchModule::OnAssetRemoved);
	FilesLoadedDelegateHandle = AssetRegistry.OnFilesLoaded().AddRaw(this, &FSemanticSearchModule::OnFilesLoaded);

	PreExitDelegateHandle = FCoreDelegates::OnEnginePreExit.AddRaw(this, &FSemanticSearchModule::SaveIndex);

	bIsInitialized = true;
}

void FSemanticSearchModule::ShutdownModule()
{
	UE_LOGF(LogSemanticSearch, Verbose, "SemanticSearch module shutting down");

	if (!bIsInitialized)
	{
		// Initialize() was never called (delayed-init mode with no explicit Initialize), so there
		// is nothing to tear down: no consumer queue running, no delegates bound, no timers set.
		return;
	}

	CancelIndexing();

	FHybridSearchIndex::Get().StopCommandQueue();

	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(DismissTimerHandle);
		GEditor->GetTimerManager()->ClearTimer(PeriodicSaveTimerHandle);
		GEditor->GetTimerManager()->ClearTimer(PendingOpTimeoutHandle);
		for (TPair<FName, FTimerHandle>& Pair : PendingPackageSavedTimers)
		{
			GEditor->GetTimerManager()->ClearTimer(Pair.Value);
		}
		PendingPackageSavedTimers.Empty();
	}

	FCoreDelegates::OnEnginePreExit.Remove(PreExitDelegateHandle);

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		IAssetRegistry& AssetRegistry = AssetRegistryModule->Get();

		// Todo Speed UP
		//AssetRegistry.OnAssetAdded().Remove(AssetAddedDelegateHandle);
		UPackage::PackageSavedWithContextEvent.Remove(PackageSavedDelegateHandle);
		AssetRegistry.OnAssetRenamed().Remove(AssetRenamedDelegateHandle);
		AssetRegistry.OnAssetRemoved().Remove(AssetRemovedDelegateHandle);
		AssetRegistry.OnFilesLoaded().Remove(FilesLoadedDelegateHandle);
	}

	bIsInitialized = false;
}

void FSemanticSearchModule::TriggerIndexing(const FAssetData& AssetData, bool bBuildOnMiss)
{
	if (!FAssetProcessorManager::Get().GetProcessorForAsset(AssetData))
	{
		return;
	}
	OnIndexingStarted();

	const uint32 CurIndexId = FHybridSearchIndex::Get().GetIndexId();
	const uint32 CapturedEpoch = IndexingEpoch;

	FHybridSearchIndex& Index = FHybridSearchIndex::Get();
	FSemanticSearchIndexStats Stats = Index.GetCachedIndexStats();
	const bool bQuantized = Stats.bSupportsQuantization && Stats.bIsTrained;

	bool bStarted;
	if (bQuantized)
	{
		FIoHash CodebookHash = Index.GetCachedCodebookHash();

		bStarted = FAssetProcessorManager::Get().GetQuantizedIndexingData(AssetData, CodebookHash,
			[Asset = FAssetData(AssetData), CurIndexId, CapturedEpoch](FAssetIndexingResult&& Result, FString&& Error, EAssetIndexFailureReason Reason) mutable
			{
				if (!Error.IsEmpty() || Result.QuantizedCodes.IsEmpty())
				{
					// Only mark failed when we have a classified reason. None = cancel / cache miss — not a failure.
					if (Reason != EAssetIndexFailureReason::None)
					{
						UE_LOGF(LogSemanticSearch, Verbose, "Failed to get quantized codes for %ls: %ls",
							*Asset.GetObjectPathString(), Error.IsEmpty() ? TEXT("empty result") : *Error);
						FHybridSearchIndex::Get().MarkFailed(GetAssetIndexID(Asset), Reason, CurIndexId);
					}
					FSemanticSearchModule::Get().OnIndexingFinished(false, CapturedEpoch, Reason);
					return;
				}

				FHybridSearchIndex::Get().AddQuantized(
					MoveTemp(Asset), MoveTemp(Result.QuantizedCodes),
					MoveTemp(Result.Caption), MoveTemp(Result.Keywords), CurIndexId);
				FSemanticSearchModule::Get().OnIndexingFinished(true, CapturedEpoch);
			}, bBuildOnMiss);
	}
	else
	{
		bStarted = FAssetProcessorManager::Get().GetIndexingData(AssetData,
			[Asset = FAssetData(AssetData), bBuildOnMiss, CurIndexId, CapturedEpoch](FAssetIndexingResult&& Result, FString&& Error, EAssetIndexFailureReason Reason) mutable
			{
				if (!Error.IsEmpty() || Result.Embedding.IsEmpty())
				{
					// Only mark failed when we have a classified reason. None = cancel / cache miss — not a failure.
					if (Reason != EAssetIndexFailureReason::None)
					{
						if (bBuildOnMiss)
						{
							UE_LOGF(LogSemanticSearch, Verbose, "Indexing failed for %ls: %ls",
								*Asset.GetObjectPathString(),
								Error.IsEmpty() ? TEXT("empty embedding") : *Error);
						}
						FHybridSearchIndex::Get().MarkFailed(GetAssetIndexID(Asset), Reason, CurIndexId);
					}
					FSemanticSearchModule::Get().OnIndexingFinished(false, CapturedEpoch, Reason);
					return;
				}

				FHybridSearchIndex::Get().Add(
					MoveTemp(Asset), MoveTemp(Result.Embedding),
					MoveTemp(Result.Caption), MoveTemp(Result.Keywords), CurIndexId);
				FSemanticSearchModule::Get().OnIndexingFinished(true, CapturedEpoch);
			}, bBuildOnMiss);
	}

	if (!bStarted)
	{
		OnIndexingFinished(false, CapturedEpoch);
	}
}

void FSemanticSearchModule::OnAssetAdded(const FAssetData& AssetData)
{
	if (AssetData.IsRedirector() || !IsInIndexedFolder(AssetData)) { return; }

	const int64 ID = GetAssetIndexID(AssetData);

	// Skip assets already known to have failed (in either bucket). The retry button — and only the
	// retry button — can pull retryable failures back into indexing; OnAssetAdded is an auto-retry
	// path that the user asked to NOT do.
	FHybridSearchIndex::Get().IsFailedAsync({ID},
		[this, AssetData = FAssetData(AssetData), ID](TSet<int64>&& FailedIDs)
		{
			if (FailedIDs.Contains(ID))
			{
				return;
			}

			if (LoadedPackageHashes.IsEmpty())
			{
				const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
				UE_LOGF(LogSemanticSearch, Verbose, "OnAssetAdded: %ls", *AssetData.GetObjectPathString());
				TriggerIndexing(AssetData, Settings && Settings->bAutoIndexUncachedOnStartup);
				return;
			}

			// Check staleness asynchronously via consumer thread
			FHybridSearchIndex::Get().ContainsAsync({ID},
				[this, AssetData, ID](TSet<int64>&& ContainedIDs)
				{
					if (ContainedIDs.Contains(ID))
					{
						// Already in index — check staleness via loaded hashes
						FIoHash StoredHash;
						if (LoadedPackageHashes.RemoveAndCopyValue(ID, StoredHash))
						{
							FIoHash CurrentHash = GetPackageSavedHash(AssetData);
							if (StoredHash == CurrentHash)
							{
								return; // Up to date, skip
							}
						}
					}

					const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
					UE_LOGF(LogSemanticSearch, Verbose, "OnAssetAdded: %ls", *AssetData.GetObjectPathString());
					TriggerIndexing(AssetData, Settings && Settings->bAutoIndexUncachedOnStartup);
				});
		});
}

void FSemanticSearchModule::OnPackageSaved(const FString& PackageFileName, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	if (!Settings || !Settings->bAutoIndexOnSave)
	{
		return;
	}

	if (!IsInIndexedFolder(Package->GetName()))
	{
		return;
	}

	FName PackageName = Package->GetFName();
	FIoHash SavedHash = Package->GetSavedHash();
	UE_LOGF(LogSemanticSearch, Verbose, "OnPackageSaved: %ls (deferring until AR matches hash)", *PackageName.ToString());

	// Defer so the AssetRegistry has time to update its data for the saved package.
	// Timer callbacks run on the game thread via FTimerManager::Tick.
	if (!GEditor)
	{
		return;
	}
	constexpr int32 InitialRetryCount = 0;
	FTimerHandle& Handle = PendingPackageSavedTimers.FindOrAdd(PackageName);
	GEditor->GetTimerManager()->SetTimer(Handle,
		FTimerDelegate::CreateRaw(this, &FSemanticSearchModule::OnPackageSavedDeferred, PackageName, SavedHash, InitialRetryCount),
		0.1f, /*bLoop=*/false);
}

void FSemanticSearchModule::OnPackageSavedDeferred(FName PackageName, FIoHash ExpectedHash, int32 RetryCount)
{
	check(IsInGameThread());
	static constexpr int32 MaxRetries = 10;

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	FAssetPackageData AssetPackageData;
	UE::AssetRegistry::EExists Result = AssetRegistry.TryGetAssetPackageData(PackageName, AssetPackageData);
	if (Result != UE::AssetRegistry::EExists::Exists || AssetPackageData.GetPackageSavedHash() != ExpectedHash)
	{
		if (RetryCount < MaxRetries)
		{
			UE_LOGF(LogSemanticSearch, Verbose, "OnPackageSavedDeferred: AR not up to date for %ls yet, retrying (%d/%d)",
				*PackageName.ToString(), RetryCount + 1, MaxRetries);
			if (GEditor)
			{
				FTimerHandle& Handle = PendingPackageSavedTimers.FindOrAdd(PackageName);
				GEditor->GetTimerManager()->SetTimer(Handle,
					FTimerDelegate::CreateRaw(this, &FSemanticSearchModule::OnPackageSavedDeferred, PackageName, ExpectedHash, RetryCount + 1),
					0.1f, /*bLoop=*/false);
			}
		}
		else
		{
			UE_LOGF(LogSemanticSearch, Verbose, "OnPackageSavedDeferred: AR hash never matched for %ls after %d retries, giving up",
				*PackageName.ToString(), MaxRetries);
			PendingPackageSavedTimers.Remove(PackageName);
		}

		return;
	}

	PendingPackageSavedTimers.Remove(PackageName);

	TArray<FAssetData> Assets;
	AssetRegistry.GetAssetsByPackageName(PackageName, Assets);
	for (const FAssetData& AssetData : Assets)
	{
		if (IsInIndexedFolder(AssetData))
		{
			UE_LOGF(LogSemanticSearch, Verbose, "OnPackageSavedDeferred: indexing %ls", *AssetData.GetObjectPathString());
			TriggerIndexing(AssetData, /*bBuildOnMiss=*/true);
		}
	}
}

void FSemanticSearchModule::OnAssetRenamed(const FAssetData& NewAssetData, const FString& OldObjectPath)
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	const bool bOldInFolder = IsInIndexedFolder(OldObjectPath);
	const bool bNewInFolder = IsInIndexedFolder(NewAssetData);

	UE_LOGF(LogSemanticSearch, Verbose, "OnAssetRenamed: %ls -> %ls (OldInFolder=%d, NewInFolder=%d)",
		*OldObjectPath, *NewAssetData.GetObjectPathString(), bOldInFolder, bNewInFolder);

	if (!bOldInFolder && !bNewInFolder)
	{
		return;
	}

	if (bOldInFolder && bNewInFolder)
	{
		// Update via the queue; the consumer thread will handle it as a no-op if not in index.
		// Also trigger indexing as a fallback for assets not yet indexed.
		UE_LOGF(LogSemanticSearch, Verbose, "OnAssetRenamed: enqueuing update for %ls -> %ls",
			*OldObjectPath, *NewAssetData.GetObjectPathString());
		FHybridSearchIndex::Get().RemoveByPath(FString(OldObjectPath));
		TriggerIndexing(NewAssetData, Settings && Settings->bAutoIndexOnSave);
	}
	else if (bOldInFolder)
	{
		UE_LOGF(LogSemanticSearch, Verbose, "OnAssetRenamed: removed %ls from index (moved out of indexed folder)",
			*OldObjectPath);
		FHybridSearchIndex::Get().RemoveByPath(FString(OldObjectPath));
	}
	else
	{
		UE_LOGF(LogSemanticSearch, Verbose, "OnAssetRenamed: moved into indexed folder, indexing %ls",
			*NewAssetData.GetObjectPathString());
		TriggerIndexing(NewAssetData, Settings && Settings->bAutoIndexOnSave);
	}
}

void FSemanticSearchModule::OnAssetRemoved(const FAssetData& AssetData)
{
	if (ISemanticSearchModule::IsInIndexedFolder(AssetData))
	{
		FHybridSearchIndex::Get().Remove(FAssetData(AssetData));
	}
}

TSharedPtr<IEmbeddingProvider> FSemanticSearchModule::GetEmbeddingProvider() const
{
	return RegisteredEmbeddingProvider;
}

void FSemanticSearchModule::RegisterEmbeddingProvider(TUniquePtr<IEmbeddingProvider>&& EmbeddingProvider, bool bRebuildIndex)
{
	check(IsInGameThread());

	if (!EmbeddingProvider)
	{
		UE_LOGF(LogSemanticSearch, Warning, "RegisterEmbeddingProvider: ignoring null provider");
		return;
	}

	// Cancel all in-flight indexing/HTTP work driven by the current provider and let the FRequestOwner cancellations propagate before swapping. 
	CancelIndexing();

	const int32 NewDimension = EmbeddingProvider->GetEmbeddingDimension();

	if (NewDimension > 0)
	{
		const int32 OldDimension = FHybridSearchIndex::Get().GetCachedIndexStats().Dimension;
		RegisteredEmbeddingProvider = MakeShareable<IEmbeddingProvider>(EmbeddingProvider.Release());

		if (bIsInitialized && bRebuildIndex)
		{
			RebuildVectorIndexAsFlat(NewDimension);
			UE_LOGF(LogSemanticSearch, Log,
				"Embedding provider replaced (NewDim=%d, OldDim=%d): rebuilt vector index as Flat",
				NewDimension, OldDimension);
		}
		// If !bIsInitialized, Initialize() will compare the loaded index's dimension to the
		// registered provider and rebuild there if they disagree.
	}
	else
	{
		UE_LOGF(LogSemanticSearch, Warning,
			"RegisterEmbeddingProvider: new provider reports dimension %d; leaving hybrid index untouched and keep the previous provider",
			NewDimension);
	}
}

void FSemanticSearchModule::RebuildVectorIndexAsFlat(int32 NewDimension)
{
	check(IsInGameThread());
	check(NewDimension > 0);

	FHybridSearchIndex& Index = FHybridSearchIndex::Get();
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	TSharedPtr<IVectorIndex> NewIndex = CreateVectorIndex(
		ESemanticSearchIndexType::Flat, NewDimension, Settings);

	// InvalidateIndexId mirrors CancelIndexing/SwitchToFlatIndex: any TriggerIndexing
	// callbacks still resolving against the old IndexId will be discarded by the
	// consumer thread before they touch the rebuilt index.
	Index.InvalidateIndexId();
	Index.SetVectorIndex(NewIndex);
	Index.ClearBM25();
	Index.ResetBM25AndFailedState();

	USemanticSearchSettings* MutableSettings = GetMutableDefault<USemanticSearchSettings>();
	if (MutableSettings && MutableSettings->IndexType != ESemanticSearchIndexType::Flat)
	{
		MutableSettings->IndexType = ESemanticSearchIndexType::Flat;
		MutableSettings->SaveConfig();
	}
}

void FSemanticSearchModule::IndexAsset(const FAssetData& AssetData)
{
	TriggerIndexing(AssetData, /*bBuildOnMiss=*/true);
}

void FSemanticSearchModule::IndexAssets(TConstArrayView<FAssetData> Assets)
{
	// Filter to assets that have a registered processor
	TArray<FAssetData> Filtered;
	Filtered.Reserve(Assets.Num());
	for (const FAssetData& Asset : Assets)
	{
		if (FAssetProcessorManager::Get().GetProcessorForAsset(Asset))
		{
			Filtered.Add(Asset);
		}
	}

	if (Filtered.IsEmpty())
	{
		// Release the pending-op flag set by a UI handler so the button can be clicked again.
		ClearIndexingPending();
		return;
	}
	ApplyHttpOverrides();
	const uint32 CurIndexId = FHybridSearchIndex::Get().GetIndexId();
	const uint32 CapturedEpoch = IndexingEpoch;
	OnBatchIndexingStarted(Filtered.Num());

	FHybridSearchIndex& Index = FHybridSearchIndex::Get();
	FSemanticSearchIndexStats Stats = Index.GetCachedIndexStats();
	const bool bQuantized = Stats.bSupportsQuantization && Stats.bIsTrained;

	if (bQuantized)
	{
		FIoHash CodebookHash = Index.GetCachedCodebookHash();

		FAssetProcessorManager::Get().BatchGetQuantizedIndexingData(Filtered, CodebookHash,
			[this, CurIndexId, CapturedEpoch](const FAssetData& Asset, FAssetIndexingResult&& Result, FString&& Error, EAssetIndexFailureReason Reason)
			{
				if (!Error.IsEmpty() || Result.QuantizedCodes.IsEmpty())
				{
					if (Reason != EAssetIndexFailureReason::None)
					{
						UE_LOGF(LogSemanticSearch, Verbose, "Failed to get quantized codes for %ls: %ls",
							*Asset.GetObjectPathString(), Error.IsEmpty() ? TEXT("empty result") : *Error);
						FHybridSearchIndex::Get().MarkFailed(GetAssetIndexID(Asset), Reason, CurIndexId);
					}
					OnIndexingFinished(false, CapturedEpoch, Reason);
					return;
				}

				FHybridSearchIndex::Get().AddQuantized(
					FAssetData(Asset), MoveTemp(Result.QuantizedCodes),
					MoveTemp(Result.Caption), MoveTemp(Result.Keywords), CurIndexId);
				OnIndexingFinished(true, CapturedEpoch);
			}, /*bBuildOnMiss=*/true);
	}
	else
	{
		FAssetProcessorManager::Get().BatchGetIndexingData(Filtered,
			[this, CurIndexId, CapturedEpoch](const FAssetData& Asset, FAssetIndexingResult&& Result, FString&& Error, EAssetIndexFailureReason Reason)
			{
				if (!Error.IsEmpty() || Result.Embedding.IsEmpty())
				{
					if (Reason != EAssetIndexFailureReason::None)
					{
						UE_LOGF(LogSemanticSearch, Verbose, "Indexing failed for %ls: %ls",
							*Asset.GetObjectPathString(),
							Error.IsEmpty() ? TEXT("empty embedding") : *Error);
						FHybridSearchIndex::Get().MarkFailed(GetAssetIndexID(Asset), Reason, CurIndexId);
					}
					OnIndexingFinished(false, CapturedEpoch, Reason);
					return;
				}

				FHybridSearchIndex::Get().Add(
					FAssetData(Asset), MoveTemp(Result.Embedding),
					MoveTemp(Result.Caption), MoveTemp(Result.Keywords), CurIndexId);
				OnIndexingFinished(true, CapturedEpoch);
			}, /*bBuildOnMiss=*/true);
	}
}

void FSemanticSearchModule::OnIndexingStarted()
{
	ClearIndexingPending();
	++IndexingTotal;
	bIndexingInProgress.store(true, std::memory_order_release);

	if (!IsRunningCommandlet())
	{
		if (!ProgressHandle.IsValid())
		{
			ProgressHandle = FSlateNotificationManager::Get().StartProgressNotification(
				FText::FromString(FString::Printf(TEXT("Processed 0/%d"), IndexingTotal)), IndexingTotal);

			if (GEditor && !ProgressUpdateTimerHandle.IsValid())
			{
				GEditor->GetTimerManager()->SetTimer(ProgressUpdateTimerHandle,
					FTimerDelegate::CreateRaw(this, &FSemanticSearchModule::UpdateProgressNotification),
					0.1f, /*bLoop=*/true);
			}
		}

		if (DismissTimerHandle.IsValid() && GEditor)
		{
			GEditor->GetTimerManager()->ClearTimer(DismissTimerHandle);
		}
	}
}

void FSemanticSearchModule::OnBatchIndexingStarted(int32 Count)
{
	ClearIndexingPending();
	IndexingTotal += Count;
	bIndexingInProgress.store(true, std::memory_order_release);

	if (!IsRunningCommandlet())
	{
		if (!ProgressHandle.IsValid())
		{
			ProgressHandle = FSlateNotificationManager::Get().StartProgressNotification(
				FText::FromString(FString::Printf(TEXT("Processed 0/%d"), IndexingTotal)), IndexingTotal);

			if (GEditor && !ProgressUpdateTimerHandle.IsValid())
			{
				GEditor->GetTimerManager()->SetTimer(ProgressUpdateTimerHandle,
					FTimerDelegate::CreateRaw(this, &FSemanticSearchModule::UpdateProgressNotification),
					0.1f, /*bLoop=*/true);
			}
		}

		if (DismissTimerHandle.IsValid() && GEditor)
		{
			GEditor->GetTimerManager()->ClearTimer(DismissTimerHandle);
		}
	}
}

void FSemanticSearchModule::OnIndexingFinished(bool bSuccess, uint32 Epoch, EAssetIndexFailureReason Reason)
{
	AsyncTask(ENamedThreads::GameThread, [this, bSuccess, Epoch, Reason]()
	{
		if (Epoch != IndexingEpoch)
		{
			return;
		}

		if (bSuccess)
		{
			++IndexingCompleted;
		}
		else if (Reason == EAssetIndexFailureReason::PreProcessor)
		{
			++IndexingPreProcessorFailed;
		}
		else if (Reason == EAssetIndexFailureReason::Provider)
		{
			++IndexingFailed;
		}
		else
		{
			// Reason == None: cancellation or cache-miss-without-buildOnMiss.
			--IndexingTotal;
		}

		const int32 DoneCount = IndexingCompleted + IndexingFailed + IndexingPreProcessorFailed;
		if (DoneCount % 50 == 0)
		{
			FHybridSearchIndex::Get().OnIndexChanged().Broadcast(true);
		}

		if (DoneCount >= IndexingTotal)
		{
			ResetHttpOverrides();
			bIndexingInProgress.store(false, std::memory_order_release);
			if (GEditor)
			{
				GEditor->GetTimerManager()->SetTimer(DismissTimerHandle,
					FTimerDelegate::CreateRaw(this, &FSemanticSearchModule::DismissProgressNotification),
					5.0f, /*bLoop=*/false);
			}
		}
	});
}

void FSemanticSearchModule::UpdateProgressNotification()
{
	if (!ProgressHandle.IsValid() || IsRunningCommandlet())
	{
		return;
	}

	const int32 DoneCount = IndexingCompleted + IndexingFailed + IndexingPreProcessorFailed;
	
	FString Text = FString::Printf(TEXT("Processed %d/%d"), DoneCount, IndexingTotal);

	FSlateNotificationManager::Get().UpdateProgressNotification(
		ProgressHandle, DoneCount, IndexingTotal,
		FText::FromString(Text));
}

void FSemanticSearchModule::DismissProgressNotification()
{
	if (IsRunningCommandlet())
	{
		IndexingTotal = 0;
		IndexingCompleted = 0;
		IndexingFailed = 0;
		IndexingPreProcessorFailed = 0;
		FHybridSearchIndex::Get().OnIndexChanged().Broadcast(true);
		return;
	}

	if (ProgressUpdateTimerHandle.IsValid() && GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(ProgressUpdateTimerHandle);
	}

	if (ProgressHandle.IsValid())
	{
		FSlateNotificationManager::Get().CancelProgressNotification(ProgressHandle);
		ProgressHandle = FProgressNotificationHandle();
	}
	
	const int32 PersistableOutcomes = IndexingCompleted + IndexingPreProcessorFailed;
	if (PersistableOutcomes >= SaveIndexThreshold)
	{
		SaveIndex();
	}

	IndexingTotal = 0;
	IndexingCompleted = 0;
	IndexingFailed = 0;
	IndexingPreProcessorFailed = 0;

	FHybridSearchIndex::Get().OnIndexChanged().Broadcast(true);
}

void FSemanticSearchModule::ResetProgressState()
{
	++IndexingEpoch;
	bIndexingInProgress.store(false, std::memory_order_release);

	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(DismissTimerHandle);
	}

	if (ProgressHandle.IsValid())
	{
		FSlateNotificationManager::Get().CancelProgressNotification(ProgressHandle);
		ProgressHandle = FProgressNotificationHandle();
	}

	IndexingTotal = 0;
	IndexingCompleted = 0;
	IndexingFailed = 0;
	ResetHttpOverrides();
	IndexingPreProcessorFailed = 0;
}

void FSemanticSearchModule::OnFilesLoaded()
{
	// Guard against OnFilesLoaded being called multiple times by the asset registry
	if (bFilesLoaded)
	{
		return;
	}
	bFilesLoaded = true;

	// Anything left in LoadedPackageHashes was not seen during OnAssetAdded — the asset was deleted
	// if (!LoadedPackageHashes.IsEmpty())
	// {
	// 	for (const auto& [ID, Hash] : LoadedPackageHashes)
	// 	{
	// 		FHybridSearchIndex::Get().RemoveById(ID);
	// 	}
	// 	UE_LOGF(LogSemanticSearch, Log, "Removed %d orphaned entries from loaded index", LoadedPackageHashes.Num());
	// 	LoadedPackageHashes.Empty();
	// }

	// Purge of stale pre-processor-failed IDs is folded into the SaveIndex pass below — that pass
	// already enumerates the AssetRegistry, so we piggyback on it instead of scanning twice.
	SaveIndex();
	// Start periodic save timer
	if (GEditor)
	{
		static constexpr float PeriodicSaveIntervalSeconds = 3600.0f; // 1 hour
		GEditor->GetTimerManager()->SetTimer(PeriodicSaveTimerHandle,
			FTimerDelegate::CreateRaw(this, &FSemanticSearchModule::SaveIndex),
			PeriodicSaveIntervalSeconds, /*bLoop=*/true);
	}

	// Auto-index on startup if enabled
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	if (Settings && Settings->bAutoIndexCachedOnStartup)
	{
		UE_LOGF(LogSemanticSearch, Log, "Auto-indexing on startup (bBuildUncached=%d)", Settings->bAutoIndexUncachedOnStartup);
		IndexAllAssets(/*bForceBuild=*/false);
	}
}

FSemanticSearchIndexStats FSemanticSearchModule::GetIndexStats() const
{
	FSemanticSearchIndexStats Stats = FHybridSearchIndex::Get().GetCachedIndexStats();

	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	Stats.IndexType = Settings ? Settings->IndexType : ESemanticSearchIndexType::Flat;
	Stats.SupportedAssetCount = FAssetProcessorManager::Get().GetSupportedAssetCount();
	const bool bActivelyIndexing = ProgressHandle.IsValid() && IndexingTotal > 0
		&& (IndexingCompleted + IndexingFailed + IndexingPreProcessorFailed) < IndexingTotal;
	// Treat the brief "click received, async work in flight but progress not started yet" window
	// as busy too, so buttons don't feel unresponsive while the op spins up.
	Stats.bIsIndexing = bActivelyIndexing || bHasPendingIndexingOp;

	return Stats;
}

void FSemanticSearchModule::SetIndexingPending()
{
	check(IsInGameThread());
	bHasPendingIndexingOp = true;

	// Safety timeout — if no indexing actually kicks off (e.g. button handler hits an empty-list
	// early return), clear the pending flag so the UI doesn't stay stuck in "busy" state.
	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(PendingOpTimeoutHandle);
		GEditor->GetTimerManager()->SetTimer(PendingOpTimeoutHandle,
			FTimerDelegate::CreateRaw(this, &FSemanticSearchModule::ClearIndexingPending),
			5.0f, /*bLoop=*/false);
	}

	// Broadcast so the CB toolbar re-evaluates IsEnabled predicates this frame.
	FHybridSearchIndex::Get().OnIndexChanged().Broadcast(true);
}

void FSemanticSearchModule::ClearIndexingPending()
{
	check(IsInGameThread());
	if (!bHasPendingIndexingOp)
	{
		return;
	}
	bHasPendingIndexingOp = false;
	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(PendingOpTimeoutHandle);
	}
	FHybridSearchIndex::Get().OnIndexChanged().Broadcast(true);
}

void FSemanticSearchModule::CancelIndexing()
{
	check(IsInGameThread());
	FHybridSearchIndex::Get().InvalidateIndexId();

	FHybridSearchIndex::Get().DiscardPendingMutations();

	FAssetProcessorManager::Get().CancelAllTasks();

	// Cancel pending HTTP requests in the embedding provider queues to prevent
	// hundreds of already-cancelled requests from being dispatched to the HTTP layer
	if (RegisteredEmbeddingProvider)
	{
		RegisteredEmbeddingProvider->CancelAllPendingRequests();
	}

	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(DismissTimerHandle);
		GEditor->GetTimerManager()->ClearTimer(ProgressUpdateTimerHandle);
	}

	if (ProgressHandle.IsValid())
	{
		FSlateNotificationManager::Get().CancelProgressNotification(ProgressHandle);
		ProgressHandle = FProgressNotificationHandle();
	}
	
	const int32 PersistableOutcomes = IndexingCompleted + IndexingPreProcessorFailed;
	if (PersistableOutcomes > 0 && !IsEngineExitRequested())
	{
		SaveIndex();
	}

	IndexingTotal = 0;
	IndexingCompleted = 0;
	IndexingFailed = 0;
	ResetHttpOverrides();
	IndexingPreProcessorFailed = 0;
	ClearIndexingPending();

	bIndexingInProgress.store(false, std::memory_order_release);

	UE_LOG(LogSemanticSearch, Log, TEXT("CancelIndexing: Canceled all tasks (IndexId=%u)"),
		FHybridSearchIndex::Get().GetIndexId());
}

void FSemanticSearchModule::SwitchIndexType(ESemanticSearchIndexType NewType)
{
	if (bIsSwitchingIndexType)
	{
		UE_LOGF(LogSemanticSearch, Warning, "SwitchIndexType: Already switching, ignoring request");
		return;
	}

	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	if (!Settings || Settings->IndexType == NewType)
	{
		return;
	}

	bIsSwitchingIndexType = true;

	if (NewType == ESemanticSearchIndexType::Flat)
	{
		SwitchToFlatIndex();
	}
	else
	{
		SwitchToQuantizedIndex(NewType);
	}
}

void FSemanticSearchModule::SwitchToFlatIndex()
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	const int32 Dimension = FHybridSearchIndex::Get().GetCachedIndexStats().Dimension;

	CancelIndexing();

	TSharedPtr<IVectorIndex> NewIndex = CreateVectorIndex(ESemanticSearchIndexType::Flat, Dimension, Settings);
	FHybridSearchIndex::Get().SetVectorIndex(NewIndex);
	FHybridSearchIndex::Get().ClearBM25();

	USemanticSearchSettings* MutableSettings = GetMutableDefault<USemanticSearchSettings>();
	MutableSettings->IndexType = ESemanticSearchIndexType::Flat;
	MutableSettings->SaveConfig();

	UE_LOGF(LogSemanticSearch, Log, "Switched to Flat index, re-indexing all assets from DDC...");
	bIsSwitchingIndexType = false;

	// Re-index all assets — TriggerIndexing fetches embeddings from DDC
	IndexAllAssets(/*bForceBuild=*/true);
}

void FSemanticSearchModule::SwitchToQuantizedIndex(ESemanticSearchIndexType NewType, bool bForceTrain)
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	FHybridSearchIndex& Index = FHybridSearchIndex::Get();
	FSemanticSearchIndexStats Stats = Index.GetCachedIndexStats();
	const int32 Dimension = Stats.Dimension;

	if (Dimension == 0)
	{
		UE_LOGF(LogSemanticSearch, Warning, "SwitchToQuantizedIndex: Current index dimension zero, cannot switch");
		bIsSwitchingIndexType = false;
		FHybridSearchIndex::Get().OnIndexChanged().Broadcast(false);
		return;
	}

	// Try to load existing codebook to skip training (unless forced)
	TSharedPtr<IVectorIndex> NewIndex;
	if (!bForceTrain)
	{
		const FString CodebookPath = FHybridSearchIndex::GetCodebookPath(NewType);
		TArray<uint8> CodebookData;
		if (FFileHelper::LoadFileToArray(CodebookData, *CodebookPath))
		{
			NewIndex = DeserializeCodebook(NewType, CodebookData, Dimension);
			if (NewIndex)
			{
				UE_LOGF(LogSemanticSearch, Log, "Loaded existing codebook from %ls, skipping training", *CodebookPath);
			}
		}
	}

	if (NewIndex)
	{
		CancelIndexing();

		FHybridSearchIndex::Get().SetVectorIndex(NewIndex);
		FHybridSearchIndex::Get().ClearBM25();

		USemanticSearchSettings* MutableSettings = GetMutableDefault<USemanticSearchSettings>();
		MutableSettings->IndexType = NewType;
		MutableSettings->SaveConfig();

		UE_LOGF(LogSemanticSearch, Log, "Switched to quantized index with existing codebook, re-indexing from DDC...");
		bIsSwitchingIndexType = false;

		// TriggerIndexing will use the quantized fast path (DDC codes) when available
		IndexAllAssets(/*bForceBuild=*/true);
	}
	else
	{
		// No codebook — need to train. Collect vectors from current index for training.
		if (Stats.VectorCount == 0)
		{
			UE_LOGF(LogSemanticSearch, Warning, "SwitchToQuantizedIndex: No vectors available for training");
			bIsSwitchingIndexType = false;
			FHybridSearchIndex::Get().OnIndexChanged().Broadcast(false);
			return;
		}

		// Build candidate ID list from asset registry on the game thread,
		// then extract vectors on the consumer thread to avoid races.
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		TArray<FAssetData> AllAssets;
		AssetRegistry.GetAllAssets(AllAssets, /*bIncludeOnlyOnDiskAssets=*/true);

		TArray<int64> CandidateIDs;
		CandidateIDs.Reserve(AllAssets.Num());
		for (const FAssetData& Asset : AllAssets)
		{
			if (!Asset.IsRedirector() && IsInIndexedFolder(Asset))
			{
				CandidateIDs.Add(GetAssetIndexID(Asset));
			}
		}

		Index.ExtractEmbeddingsAsync(MoveTemp(CandidateIDs),
			[this, NewType, Dimension, Settings](TArray<int64>&& AllIDs, TArray<float>&& TrainingVectors, int32 Dim)
			{
				if (AllIDs.Num() == 0)
				{
					UE_LOGF(LogSemanticSearch, Warning, "SwitchToQuantizedIndex: Could not retrieve any vectors for training");
					bIsSwitchingIndexType = false;
					FHybridSearchIndex::Get().OnIndexChanged().Broadcast(false);
					return;
				}

				TSharedPtr<IVectorIndex> TrainedIndex = CreateVectorIndex(NewType, Dimension, Settings);

				UE_LOGF(LogSemanticSearch, Log, "Training quantized index (type=%d) with %d vectors (dim=%d)...",
					static_cast<uint8>(NewType), AllIDs.Num(), Dimension);

				FProgressNotificationHandle SwitchProgress = FSlateNotificationManager::Get().StartProgressNotification(
					NSLOCTEXT("SemanticSearch", "TrainingIndex", "Training quantized index..."), 0);

				// Train on background thread, then swap and re-index from DDC on game thread
				Async(EAsyncExecution::ThreadPool,
					[this, NewType, TrainedIndex, TrainingVectors = MoveTemp(TrainingVectors), NumVectors = AllIDs.Num(), SwitchProgress]() mutable
					{
						TrainedIndex->Train(TrainingVectors, NumVectors);

						auto* QuantizedIndex = static_cast<IQuantizedVectorIndex*>(TrainedIndex.Get());

						// Save codebook
						const FString CodebookPath = FHybridSearchIndex::GetCodebookPath(NewType);
						if (!CodebookPath.IsEmpty())
						{
							TArray<uint8> CodebookData = QuantizedIndex->SerializeCodebook();
							FString CodebookDir = FPaths::GetPath(CodebookPath);
							IFileManager::Get().MakeDirectory(*CodebookDir, /*bTree=*/true);
							if (FFileHelper::SaveArrayToFile(CodebookData, *CodebookPath))
							{
								UE_LOGF(LogSemanticSearch, Log, "Saved codebook to %ls", *CodebookPath);
							}
							else
							{
								UE_LOGF(LogSemanticSearch, Warning, "Failed to save codebook to %ls", *CodebookPath);
							}
						}

						// Finalize on game thread — swap index and re-index all assets from DDC
						AsyncTask(ENamedThreads::GameThread,
							[this, NewType, TrainedIndex = MoveTemp(TrainedIndex), NumVectors, SwitchProgress]() mutable
							{
								CancelIndexing();

								FHybridSearchIndex::Get().SetVectorIndex(MoveTemp(TrainedIndex));
								FHybridSearchIndex::Get().ClearBM25();

								USemanticSearchSettings* MutableSettings = GetMutableDefault<USemanticSearchSettings>();
								MutableSettings->IndexType = NewType;
								MutableSettings->SaveConfig();
								FSlateNotificationManager::Get().CancelProgressNotification(SwitchProgress);
								bIsSwitchingIndexType = false;

						UE_LOGF(LogSemanticSearch, Log, "Quantized index trained, re-indexing %d assets from DDC...", NumVectors);

						IndexAllAssets(/*bForceBuild=*/true);
					});
			});
		});
	}
}

void FSemanticSearchModule::IndexAllAssets(bool bForceBuild)
{
	ResetProgressState();
	
	if (bForceBuild)
	{
		FHybridSearchIndex::Get().ResetBM25AndFailedState();
		
		// Clear cached build tasks so fresh DDC lookups are triggered
		FAssetProcessorManager::Get().ClearBuildCache();

		// Clear and recreate the vector index so count drops to 0 and rebuilds
		FHybridSearchIndex& Index = FHybridSearchIndex::Get();
		const int32 Dimension = Index.GetCachedIndexStats().Dimension;
		if (Dimension > 0)
		{
			const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
			const ESemanticSearchIndexType Type = Settings ? Settings->IndexType : ESemanticSearchIndexType::Flat;

			// Try to reload trained codebook for PQ types
			TSharedPtr<IVectorIndex> NewIndex;
			const FString CodebookPath = FHybridSearchIndex::GetCodebookPath(Type);
			if (!CodebookPath.IsEmpty())
			{
				TArray<uint8> CodebookData;
				if (FFileHelper::LoadFileToArray(CodebookData, *CodebookPath))
				{
					NewIndex = DeserializeCodebook(Type, CodebookData, Dimension);
				}
			}
			if (!NewIndex)
			{
				NewIndex = CreateVectorIndex(Type, Dimension, Settings);
			}
			Index.SetVectorIndex(NewIndex);
		}
	}

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	TArray<FAssetData> AllAssets;
	AssetRegistry.GetAllAssets(AllAssets, /*bIncludeOnlyOnDiskAssets=*/true);

	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	const bool bBuildOnMiss = bForceBuild || (Settings && Settings->bAutoIndexUncachedOnStartup);

	// Filter to supported, indexed assets
	TArray<FAssetData> AssetsToIndex;
	AssetsToIndex.Reserve(AllAssets.Num());
	for (const FAssetData& Asset : AllAssets)
	{
		if (!Asset.IsRedirector() && IsInIndexedFolder(Asset)
			&& FAssetProcessorManager::Get().GetProcessorForAsset(Asset))
		{
			AssetsToIndex.Add(Asset);
		}
	}

	if (AssetsToIndex.IsEmpty())
	{
		ClearIndexingPending();
		FHybridSearchIndex::Get().OnIndexChanged().Broadcast(true);
		return;
	}
	ApplyHttpOverrides();

	// Drop previously-failed assets from the auto-retry path. Retryable failures were cleared on
	// restart (not persisted) so those are naturally absent; pre-processor failures persist and
	// are deliberately skipped here — only explicit user actions (force-rebuild / switch / retrain)
	// clear the pre-processor bucket. Skip this filter when bForceBuild is true — the caller has
	// already called ResetBM25AndFailedState which empties both buckets.
	if (!bForceBuild)
	{
		TArray<int64> CandidateIDs;
		CandidateIDs.Reserve(AssetsToIndex.Num());
		for (const FAssetData& Asset : AssetsToIndex)
		{
			CandidateIDs.Add(GetAssetIndexID(Asset));
		}

		FHybridSearchIndex::Get().IsFailedAsync(MoveTemp(CandidateIDs),
			[this, AssetsToIndex = MoveTemp(AssetsToIndex), bBuildOnMiss](TSet<int64>&& FailedIDs) mutable
			{
				TArray<FAssetData> Remaining;
				Remaining.Reserve(AssetsToIndex.Num() - FailedIDs.Num());
				for (const FAssetData& Asset : AssetsToIndex)
				{
					if (!FailedIDs.Contains(GetAssetIndexID(Asset)))
					{
						Remaining.Add(Asset);
					}
				}
				IndexAssetsBatch(MoveTemp(Remaining), bBuildOnMiss);
			});
		return;
	}

	IndexAssetsBatch(MoveTemp(AssetsToIndex), bBuildOnMiss);
}

void FSemanticSearchModule::IndexAssetsBatch(TArray<FAssetData> AssetsToIndex, bool bBuildOnMiss)
{
	if (AssetsToIndex.IsEmpty())
	{
		ClearIndexingPending();
		FHybridSearchIndex::Get().OnIndexChanged().Broadcast(true);
		return;
	}

	const uint32 CurIndexId = FHybridSearchIndex::Get().GetIndexId();
	const uint32 CurEpoch = IndexingEpoch;
	OnBatchIndexingStarted(AssetsToIndex.Num());

	FHybridSearchIndex& Index = FHybridSearchIndex::Get();
	FSemanticSearchIndexStats IndexStats = Index.GetCachedIndexStats();
	const bool bQuantized = IndexStats.bSupportsQuantization && IndexStats.bIsTrained;

	if (bQuantized)
	{
		FIoHash CodebookHash = Index.GetCachedCodebookHash();

		FAssetProcessorManager::Get().BatchGetQuantizedIndexingData(AssetsToIndex, CodebookHash,
			[this, CurIndexId, CurEpoch](const FAssetData& Asset, FAssetIndexingResult&& Result, FString&& Error, EAssetIndexFailureReason Reason)
			{
				if (!Error.IsEmpty() || Result.QuantizedCodes.IsEmpty())
				{
					if (Reason != EAssetIndexFailureReason::None)
					{
						UE_LOGF(LogSemanticSearch, Verbose, "Failed to get quantized codes for %ls: %ls",
							*Asset.GetObjectPathString(), Error.IsEmpty() ? TEXT("empty result") : *Error);
						FHybridSearchIndex::Get().MarkFailed(GetAssetIndexID(Asset), Reason, CurIndexId);
					}
					OnIndexingFinished(false, CurEpoch, Reason);
					return;
				}

				FHybridSearchIndex::Get().AddQuantized(
					FAssetData(Asset), MoveTemp(Result.QuantizedCodes),
					MoveTemp(Result.Caption), MoveTemp(Result.Keywords), CurIndexId);
				OnIndexingFinished(true, CurEpoch);
			}, bBuildOnMiss);
	}
	else
	{
		FAssetProcessorManager::Get().BatchGetIndexingData(AssetsToIndex,
			[this, CurIndexId, bBuildOnMiss, CurEpoch](const FAssetData& Asset, FAssetIndexingResult&& Result, FString&& Error, EAssetIndexFailureReason Reason)
			{
				if (!Error.IsEmpty() || Result.Embedding.IsEmpty())
				{
					if (Reason != EAssetIndexFailureReason::None)
					{
						if (bBuildOnMiss)
						{
							if (IsPreProcessorFailure(Reason))
							{
								UE_LOG(LogSemanticSearch, Verbose, TEXT("Indexing failed for %s: %s"),
									*Asset.GetObjectPathString(),
									Error.IsEmpty() ? TEXT("empty embedding") : *Error);
							}
							else
							{
								UE_LOG(LogSemanticSearch, Warning, TEXT("Indexing failed for %s: %s"),
									*Asset.GetObjectPathString(),
									Error.IsEmpty() ? TEXT("empty embedding") : *Error);
							}
						}
						FHybridSearchIndex::Get().MarkFailed(GetAssetIndexID(Asset), Reason, CurIndexId);
					}
					OnIndexingFinished(false, CurEpoch, Reason);
					return;
				}

				FHybridSearchIndex::Get().Add(
					FAssetData(Asset), MoveTemp(Result.Embedding),
					MoveTemp(Result.Caption), MoveTemp(Result.Keywords), CurIndexId);
				OnIndexingFinished(true, CurEpoch);
			}, bBuildOnMiss);
	}
}

void FSemanticSearchModule::RetrainIndex()
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	if (!Settings || Settings->IndexType == ESemanticSearchIndexType::Flat)
	{
		UE_LOGF(LogSemanticSearch, Warning, "RetrainIndex: Current index is Flat, nothing to retrain");
		return;
	}

	if (bIsSwitchingIndexType)
	{
		UE_LOGF(LogSemanticSearch, Warning, "RetrainIndex: Already switching/retraining, ignoring request");
		return;
	}

	bIsSwitchingIndexType = true;
	SwitchToQuantizedIndex(Settings->IndexType, /*bForceTrain=*/true);
}

void FSemanticSearchModule::SaveIndex()
{
	// Commandlet runs push to DDC only; no local .bin needed.
	if (IsRunningCommandlet())
	{
		return;
	}

	FHybridSearchIndex& Index = FHybridSearchIndex::Get();
	if (!Index.GetCachedIndexStats().bIsInitialized)
	{
		return;
	}

	// Collect all candidate asset IDs and their package hashes on the game thread,
	// then filter to indexed assets via ContainsAsync on the consumer thread.
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	TArray<int64> CandidateIDs;
	TMap<int64, FIoHash> AllHashes;
	// Also collect currently-valid IDs in indexed folders so we can purge stale pre-processor
	// failures in the same pass — avoids a second AR enumeration.
	TSet<int64> IndexedFolderIDs;
	AssetRegistry.EnumerateAllAssets([&CandidateIDs, &AllHashes, &IndexedFolderIDs, &AssetRegistry](const FAssetData& AssetData)
	{
		const int64 ID = GetAssetIndexID(AssetData);
		CandidateIDs.Add(ID);
		TOptional<FAssetPackageData> PkgData = AssetRegistry.GetAssetPackageDataCopy(AssetData.PackageName);
		if (PkgData)
		{
			AllHashes.Add(ID, PkgData->GetPackageSavedHash());
		}
		if (!AssetData.IsRedirector() && ISemanticSearchModule::IsInIndexedFolder(AssetData))
		{
			IndexedFolderIDs.Add(ID);
		}
		return true;
	});

	// Drop pre-processor-failed IDs whose assets no longer exist in indexed folders (deletions /
	// renames). Ordering note: this mutation is enqueued NOW on the game thread, before the
	// ContainsAsync below. The Save mutation is enqueued LATER from inside ContainsAsync's
	// game-thread callback. Since the consumer processes its mutation queue in FIFO order, Purge
	// is guaranteed to run before Save serializes the set. If the order of enqueuing ever flips,
	// the sidecar could serialize stale entries — keep Purge before ContainsAsync.
	FHybridSearchIndex::Get().PurgePreProcessorFailedNotIn(MoveTemp(IndexedFolderIDs));

	Index.ContainsAsync(MoveTemp(CandidateIDs),
		[AllHashes = MoveTemp(AllHashes)](TSet<int64>&& ContainedIDs)
		{
			TMap<int64, FIoHash> PackageHashes;
			PackageHashes.Reserve(ContainedIDs.Num());
			for (int64 ID : ContainedIDs)
			{
				if (const FIoHash* Hash = AllHashes.Find(ID))
				{
					PackageHashes.Add(ID, *Hash);
				}
			}

			const FString IndexPath = GetSavedIndexPath();
			const FString FailedPath = GetPreProcessorFailedPath();
			FHybridSearchIndex::Get().Save(IndexPath, FailedPath, MoveTemp(PackageHashes));
		});
}

void FSemanticSearchModule::SearchAsync(
	const FString& QueryText,
	TConstArrayView<float> QueryEmbedding,
	int32 K,
	TConstArrayView<int64> IDFilter,
	float DistanceCutoff,
	TFunction<void(TArray<FHybridSearchResult>&&)> Callback)
{
	FHybridSearchIndex::Get().SearchAsync(
		FString(QueryText),
		TArray<float>(QueryEmbedding),
		K,
		TArray<int64>(IDFilter),
		DistanceCutoff,
		MoveTemp(Callback));
}

}
