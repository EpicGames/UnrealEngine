// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISemanticSearchModule.h"

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "AssetRegistry/AssetData.h"
#include "IO/IoHash.h"
#include "TimerManager.h"

#include <atomic>

class UPackage;
class FObjectPostSaveContext;

DECLARE_LOG_CATEGORY_EXTERN(LogSemanticSearch, Log, All);

namespace UE::SemanticSearch::Private
{

class FSemanticSearchModule : public ISemanticSearchModule
{
public:
	static FSemanticSearchModule& Get();
	virtual TSharedPtr<IEmbeddingProvider> GetEmbeddingProvider() const override;

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual void RegisterEmbeddingProvider(TUniquePtr<IEmbeddingProvider>&& EmbeddingProvider, bool bRebuildIndex) override;
	virtual void Initialize() override;
	virtual bool IsInitialized() const override { return bIsInitialized; }
	virtual void IndexAsset(const FAssetData& AssetData) override;
	virtual void IndexAssets(TConstArrayView<FAssetData> Assets) override;
	virtual FSemanticSearchIndexStats GetIndexStats() const override;
	virtual void SwitchIndexType(ESemanticSearchIndexType NewType) override;
	virtual void IndexAllAssets(bool bForceBuild = false) override;
	virtual void RetrainIndex() override;
	virtual void CancelIndexing() override;
	virtual void SetIndexingPending() override;
	virtual bool IsIndexingInProgress() const override;
	virtual void SearchAsync(const FString& QueryText, TConstArrayView<float> QueryEmbedding,
		int32 K, TConstArrayView<int64> IDFilter, float DistanceCutoff,
		TFunction<void(TArray<FHybridSearchResult>&&)> Callback) override;

private:
	void SwitchToFlatIndex();
	void SwitchToQuantizedIndex(ESemanticSearchIndexType NewType, bool bForceTrain = false);
	void TriggerIndexing(const FAssetData& AssetData, bool bBuildOnMiss);

	/** Discard the current vector index and BM25 state and create a fresh Flat index sized to
	 *  NewDimension. Pins USemanticSearchSettings::IndexType to Flat so the choice survives a
	 *  restart. Caller is responsible for clearing LoadedPackageHashes when those no longer apply. */
	void RebuildVectorIndexAsFlat(int32 NewDimension);

	/** Dispatch a filtered batch to the processor manager (flat or quantized). Extracted from IndexAllAssets
	 *  so the async failed-set filter can call it after the IsFailedAsync callback fires. */
	void IndexAssetsBatch(TArray<FAssetData> AssetsToIndex, bool bBuildOnMiss);

	void OnAssetAdded(const FAssetData& AssetData);
	void OnPackageSaved(const FString& PackageFileName, UPackage* Package, FObjectPostSaveContext ObjectSaveContext);
	void OnPackageSavedDeferred(FName PackageName, FIoHash ExpectedHash, int32 RetryCount);
	void OnAssetRenamed(const FAssetData& NewAssetData, const FString& OldObjectPath);
	void OnAssetRemoved(const FAssetData& AssetData);
	void OnFilesLoaded();

	void OnIndexingStarted();
	void OnBatchIndexingStarted(int32 Count);
	/** Reason is None on success and on "not-really-failed" outcomes (cancel, cache miss without build);
	 *  PreProcessor / Provider on actual failures. The progress notification splits PreProcessor out of
	 *  "Failed" so the progress chip reconciles with the persistent retryable-only FailedCount. */
	void OnIndexingFinished(bool bSuccess, uint32 Epoch, EAssetIndexFailureReason Reason = EAssetIndexFailureReason::None);
	void UpdateProgressNotification();
	void DismissProgressNotification();
	void ResetProgressState();
	void SaveIndex();

	static FString GetSavedIndexPath();
	static FString GetPreProcessorFailedPath();

	TSharedPtr<IEmbeddingProvider> RegisteredEmbeddingProvider;

	/** True after Initialize() has loaded the index, started the consumer queue, and bound delegates. */
	bool bIsInitialized = false;

	FDelegateHandle AssetAddedDelegateHandle;
	FDelegateHandle PackageSavedDelegateHandle;
	FDelegateHandle AssetRenamedDelegateHandle;
	FDelegateHandle AssetRemovedDelegateHandle;
	FDelegateHandle FilesLoadedDelegateHandle;
	FDelegateHandle PreExitDelegateHandle;

	FProgressNotificationHandle ProgressHandle;
	int32 IndexingTotal = 0;
	int32 IndexingCompleted = 0;
	/** Retryable failures (Provider). Progress chip shows this under "Failed". */
	int32 IndexingFailed = 0;
	/** Pre-processor failures (permanent). Progress chip shows this under "Errors" separately. */
	int32 IndexingPreProcessorFailed = 0;
	uint32 IndexingEpoch = 0;
	FTimerHandle DismissTimerHandle;
	FTimerHandle ProgressUpdateTimerHandle;
	FTimerHandle PeriodicSaveTimerHandle;
	FTimerHandle PendingOpTimeoutHandle;

	/** Set synchronously by UI handlers on click; makes bIsIndexing true before async indexing actually
	 *  starts, so buttons disable visibly at click time. Cleared in OnIndexingStarted/OnBatchIndexingStarted,
	 *  by a safety timer, or by ClearIndexingPending. Game-thread only. */
	bool bHasPendingIndexingOp = false;

	void ClearIndexingPending();

	/** Guards against concurrent SwitchIndexType calls. Only accessed on game thread. */
	bool bIsSwitchingIndexType = false;


	/** Minimum number of completed indexing operations before auto-saving on dismiss. */
	static constexpr int32 SaveIndexThreshold = 100;

	/** Package hashes loaded from saved index, used for staleness checks during startup. */
	TMap<int64, FIoHash> LoadedPackageHashes;
	TMap<FName, FTimerHandle> PendingPackageSavedTimers;
	bool bFilesLoaded = false;

	/** Tracks whether a batch indexing operation is in flight (thread-safe, Slate-independent). */
	std::atomic<bool> bIndexingInProgress{false};
};
}
