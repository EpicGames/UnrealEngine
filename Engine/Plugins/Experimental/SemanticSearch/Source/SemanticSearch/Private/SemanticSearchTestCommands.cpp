// Copyright Epic Games, Inc. All Rights Reserved.

#include "SemanticSearchModule.h"
#include "HybridSearchIndex.h"
#include "AssetProcessorManager.h"
#include "AssetProcessors/AssetProcessorUtils.h"
#include "ISemanticSearchModule.h"
#include "Interfaces/IEmbeddingProvider.h"
#include "HAL/ConsoleManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Settings/SemanticSearchSettings.h"

namespace UE::SemanticSearch::Private
{

// Returns all assets in the folders configured in Semantic Search settings.
static TArray<FAssetData> GetAssetsInIndexedFolders(IAssetRegistry& AssetRegistry)
{
	const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
	if (!Settings || Settings->IndexedPaths.IsEmpty())
	{
		UE_LOGF(LogSemanticSearch, Warning, "GetAssetsInIndexedFolders: no indexed paths configured in Semantic Search settings");
		return {};
	}

	TArray<FAssetData> Result;
	TSet<FSoftObjectPath> Seen;
	for (const FDirectoryPath& Dir : Settings->IndexedPaths)
	{
		TArray<FAssetData> FolderAssets;
		AssetRegistry.GetAssetsByPath(FName(*Dir.Path), FolderAssets, /*bRecursive=*/true);
		int32 Added = 0;
		for (FAssetData& Asset : FolderAssets)
		{
			if (!Seen.Contains(Asset.GetSoftObjectPath()))
			{
				Seen.Add(Asset.GetSoftObjectPath());
				Result.Add(MoveTemp(Asset));
				++Added;
			}
		}
		UE_LOGF(LogSemanticSearch, Log, "GetAssetsInIndexedFolders: '%ls' -> %d assets (%d new)", *Dir.Path, FolderAssets.Num(), Added);
	}
	return Result;
}

/** Test commands run against the module's configured provider so they reflect the user's settings. */
static TSharedPtr<IEmbeddingProvider> GetOrCreateProvider()
{
	return ISemanticSearchModule::Get().GetEmbeddingProvider();
}

// SemanticSearch.IndexStats
static FAutoConsoleCommand IndexStatsCommand(
	TEXT("SemanticSearch.IndexStats"),
	TEXT("Print the number of assets currently in the vector index."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FHybridSearchIndex& Searcher = FHybridSearchIndex::Get();
		FSemanticSearchIndexStats Stats = Searcher.GetCachedIndexStats();
		if (!Stats.bIsInitialized)
		{
			UE_LOGF(LogSemanticSearch, Warning, "No vector index is registered.");
			return;
		}

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		TArray<FAssetData> Assets = GetAssetsInIndexedFolders(AssetRegistry);

		TArray<int64> IDs;
		IDs.Reserve(Assets.Num());
		for (const FAssetData& Asset : Assets)
		{
			IDs.Add(GetAssetIndexID(Asset));
		}
		const int32 Total = Assets.Num();

		Searcher.ContainsAsync(MoveTemp(IDs),
			[Total, Stats](TSet<int64>&& ContainedIDs)
			{
				const int32 Indexed = ContainedIDs.Num();
				UE_LOGF(LogSemanticSearch, Log, "Index: %d entries | Folder assets: %d total, %d indexed, %d not indexed (%d dims)",
					Stats.VectorCount, Total, Indexed, Total - Indexed, Stats.Dimension);
			});
	})
);

// SemanticSearch.IndexAll
static FAutoConsoleCommand IndexAllCommand(
	TEXT("SemanticSearch.IndexAll"),
	TEXT("Index all assets in the configured folders that are not yet in the FAISS index."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FHybridSearchIndex& Searcher = FHybridSearchIndex::Get();
		if (!Searcher.GetCachedIndexStats().bIsInitialized)
		{
			UE_LOGF(LogSemanticSearch, Warning, "No vector index is registered.");
			return;
		}

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		TArray<FAssetData> Assets = GetAssetsInIndexedFolders(AssetRegistry);
		UE_LOGF(LogSemanticSearch, Log, "IndexAll: found %d assets in indexed folders", Assets.Num());

		TArray<int64> IDs;
		IDs.Reserve(Assets.Num());
		for (const FAssetData& Asset : Assets)
		{
			IDs.Add(GetAssetIndexID(Asset));
		}

		Searcher.ContainsAsync(MoveTemp(IDs),
			[Assets = MoveTemp(Assets)](TSet<int64>&& ContainedIDs)
			{
				TArray<FAssetData> ToIndex;
				int32 AlreadyIndexed = 0;
				for (const FAssetData& Asset : Assets)
				{
					if (ContainedIDs.Contains(GetAssetIndexID(Asset)))
					{
						++AlreadyIndexed;
					}
					else
					{
						ToIndex.Add(Asset);
					}
				}
				UE_LOGF(LogSemanticSearch, Log, "IndexAll: queuing %d assets for indexing, %d already indexed",
					ToIndex.Num(), AlreadyIndexed);
				ISemanticSearchModule::Get().IndexAssets(ToIndex);
			});
	})
);

// SemanticSearch.TestSearch <query text> [K]
static FAutoConsoleCommand TestSearchCommand(
	TEXT("SemanticSearch.TestSearch"),
	TEXT("Search the hybrid index. Usage: SemanticSearch.TestSearch <query text> [K=10]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 1)
		{
			UE_LOGF(LogSemanticSearch, Warning, "Usage: SemanticSearch.TestSearch <query text> [K=10]");
			return;
		}

		// If the last arg is a number, treat it as K
		int32 K = 10;
		int32 QueryArgCount = Args.Num();
		if (Args.Num() >= 2)
		{
			const int32 MaybeK = FCString::Atoi(*Args.Last());
			if (MaybeK > 0 && Args.Last() == FString::FromInt(MaybeK))
			{
				K = MaybeK;
				QueryArgCount = Args.Num() - 1;
			}
		}

		const FString QueryText = FString::Join(TConstArrayView<FString>(Args).Left(QueryArgCount), TEXT(" "));
		UE_LOGF(LogSemanticSearch, Log, "Searching for: \"%ls\" (K=%d)", *QueryText, K);

		// Build ID -> asset path map from all configured indexed folders
		TMap<int64, FString> IDToPath;
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		TArray<FAssetData> Assets = GetAssetsInIndexedFolders(AssetRegistry);
		for (const FAssetData& AssetData : Assets)
		{
			IDToPath.Add(GetAssetIndexID(AssetData), AssetData.GetObjectPathString());
		}

		TSharedPtr<IEmbeddingProvider> Provider = GetOrCreateProvider();
		if (!Provider.IsValid())
		{
			UE_LOGF(LogSemanticSearch, Warning, "No embedding provider registered.");
			return;
		}
		Provider->GenerateEmbeddingAsync(QueryText,
			[K, QueryText, IDToPath = MoveTemp(IDToPath)](const FEmbeddingResponse& Response)
			{
				if (!Response.ErrorMessage.IsEmpty())
				{
					UE_LOGF(LogSemanticSearch, Error, "Embedding failed: %ls", *Response.ErrorMessage);
					return;
				}

				FHybridSearchIndex::Get().SearchAsync(
					FString(QueryText), TArray<float>(Response.Embedding), K, {}, TNumericLimits<float>::Max(),
					[IDToPath](TArray<FHybridSearchResult>&& Results)
					{
						UE_LOGF(LogSemanticSearch, Log, "Top %d results:", Results.Num());
						for (int32 i = 0; i < Results.Num(); ++i)
						{
							const FString* Path = IDToPath.Find(Results[i].ID);
							UE_LOGF(LogSemanticSearch, Log, "  [%d] rrf=%.4f  vec_dist=%.4f  bm25=%.4f  %ls",
								i + 1,
								Results[i].RRFScore,
								Results[i].VectorDistance,
								Results[i].BM25Score,
								Path ? **Path : TEXT("<unknown>"));
						}
					});
			}, nullptr);
	})
);

// SemanticSearch.TestCaption <image_file_path>
static FAutoConsoleCommand TestCaptionCommand(
	TEXT("SemanticSearch.TestCaption"),
	TEXT("Test caption generation from an image file. Usage: SemanticSearch.TestCaption <image_file_path>"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 1)
		{
			UE_LOGF(LogSemanticSearch, Warning, "Usage: SemanticSearch.TestCaption <image_file_path>");
			return;
		}

		const FString FilePath = Args[0];
		if (!FPaths::FileExists(FilePath))
		{
			UE_LOGF(LogSemanticSearch, Error, "File not found: %ls", *FilePath);
			return;
		}

		TArray<uint8> FileData;
		if (!FFileHelper::LoadFileToArray(FileData, *FilePath))
		{
			UE_LOGF(LogSemanticSearch, Error, "Failed to load file: %ls", *FilePath);
			return;
		}

		const FString Ext = FPaths::GetExtension(FilePath).ToLower();
		FString MimeType = TEXT("application/octet-stream");
		if (Ext == TEXT("png"))              MimeType = TEXT("image/png");
		else if (Ext == TEXT("jpg") || Ext == TEXT("jpeg")) MimeType = TEXT("image/jpeg");
		else if (Ext == TEXT("bmp"))         MimeType = TEXT("image/bmp");

		FCaptionRequest Request;
		FAssetMedia Media;
		Media.MimeType = MimeType;
		Media.Data = MoveTemp(FileData);
		Request.AssetMedia.Add(MoveTemp(Media));

		UE_LOGF(LogSemanticSearch, Log, "Sending caption request (%ls, %d bytes)...", *MimeType, Request.AssetMedia[0].Data.Num());

		TSharedPtr<IEmbeddingProvider> Provider = GetOrCreateProvider();
		if (!Provider.IsValid())
		{
			UE_LOGF(LogSemanticSearch, Warning, "No embedding provider registered.");
			return;
		}
		Provider->GenerateCaptionAsync(MoveTemp(Request),
			[](const FCaptionResponse& Response)
			{
				if (!Response.ErrorMessage.IsEmpty())
				{
					UE_LOGF(LogSemanticSearch, Error, "Caption failed: %ls", *Response.ErrorMessage);
					return;
				}

				UE_LOGF(LogSemanticSearch, Log, "Caption: %ls", *Response.Caption);
				if (Response.Keywords.Num() > 0)
				{
					UE_LOGF(LogSemanticSearch, Log, "Keywords: %ls", *FString::Join(Response.Keywords, TEXT(", ")));
				}
			}, nullptr);
	})
);

// SemanticSearch.TestEmbedding [text]
static FAutoConsoleCommand TestEmbeddingCommand(
	TEXT("SemanticSearch.TestEmbedding"),
	TEXT("Test embedding generation. Usage: SemanticSearch.TestEmbedding [text to embed]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FString Text;
		if (Args.Num() > 0)
		{
			Text = FString::Join(Args, TEXT(" "));
		}
		else
		{
			Text = TEXT("A red wooden chair with four legs");
		}

		UE_LOGF(LogSemanticSearch, Log, "Generating embedding for: \"%ls\"", *Text);

		TSharedPtr<IEmbeddingProvider> Provider = GetOrCreateProvider();
		if (!Provider.IsValid())
		{
			UE_LOGF(LogSemanticSearch, Warning, "No embedding provider registered.");
			return;
		}
		Provider->GenerateEmbeddingAsync(Text,
			[](const FEmbeddingResponse& Response)
			{
				if (!Response.ErrorMessage.IsEmpty())
				{
					UE_LOGF(LogSemanticSearch, Error, "Embedding failed: %ls", *Response.ErrorMessage);
					return;
				}

				UE_LOGF(LogSemanticSearch, Log, "Embedding: %d dimensions", Response.Embedding.Num());
				if (!Response.ModelVersion.IsEmpty())
				{
					UE_LOGF(LogSemanticSearch, Log, "Model: %ls", *Response.ModelVersion);
				}

				if (Response.Embedding.Num() > 0)
				{
					const int32 Preview = FMath::Min(Response.Embedding.Num(), 5);
					FString Vals;
					for (int32 i = 0; i < Preview; ++i)
					{
						if (i > 0) Vals += TEXT(", ");
						Vals += FString::Printf(TEXT("%.6f"), Response.Embedding[i]);
					}
					UE_LOGF(LogSemanticSearch, Log, "First %d values: [%ls, ...]", Preview, *Vals);
				}
			}, nullptr);
	})
);

// SemanticSearch.ShowAssetCaption <asset object path>
static FAutoConsoleCommand ShowAssetCaptionCommand(
	TEXT("SemanticSearch.ShowAssetCaption"),
	TEXT("Print the indexed caption + keywords for an asset. Usage: SemanticSearch.ShowAssetCaption <asset object path>"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() != 1)
		{
			UE_LOGF(LogSemanticSearch, Warning, "Usage: SemanticSearch.ShowAssetCaption <asset object path>");
			return;
		}

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(Args[0]));
		if (!AssetData.IsValid())
		{
			UE_LOGF(LogSemanticSearch, Warning, "Asset not found: %ls", *Args[0]);
			return;
		}

		const bool bDispatched = FAssetProcessorManager::Get().GetIndexingData(AssetData,
			[AssetPath = Args[0]](FAssetIndexingResult&& Result, FString&& Error, EAssetIndexFailureReason Reason) mutable
			{
				if (!Error.IsEmpty())
				{
					UE_LOGF(LogSemanticSearch, Warning, "No indexed caption for %ls (%ls)", *AssetPath, *Error);
					return;
				}
				UE_LOGF(LogSemanticSearch, Log, "Caption: %ls", Result.Caption.IsEmpty() ? TEXT("<empty>") : *Result.Caption);
				if (Result.Keywords.Num() > 0)
				{
					UE_LOGF(LogSemanticSearch, Log, "Keywords: %ls", *FString::Join(Result.Keywords, TEXT(", ")));
				}
			}, /*bBuildOnMiss=*/false);

		if (!bDispatched)
		{
			UE_LOGF(LogSemanticSearch, Warning, "No processor for asset type '%ls'", *AssetData.AssetClassPath.ToString());
		}
	})
);

}
