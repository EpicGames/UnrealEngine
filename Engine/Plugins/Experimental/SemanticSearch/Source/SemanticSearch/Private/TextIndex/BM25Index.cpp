// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextIndex/BM25Index.h"

#include "Algo/Sort.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/HashTable.h"
#include "Implementations/SemanticSearchImplementationUtils.h"
#include "Misc/Crc.h"
#include "Misc/Char.h"

namespace UE::SemanticSearch
{

struct FBM25IndexHeader
{
	static constexpr uint32 Magic = 0x424D3235; // "BM25"
	static constexpr uint32 Version = 1;
};

// ---------------------------------------------------------------------------
// Varint encoding for delta-compressed DocIDs
// ---------------------------------------------------------------------------

namespace Varint
{

void Write(TArray<uint8>& Buffer, uint64 Value)
{
	while (Value >= 0x80)
	{
		Buffer.Add(static_cast<uint8>(Value & 0x7F) | 0x80);
		Value >>= 7;
	}
	Buffer.Add(static_cast<uint8>(Value));
}

// Raw-pointer variant: caller is responsible for ensuring >= 10 bytes of capacity
// from Cursor. Returns the cursor advanced past the last byte written. Used inside
// Encode where we AddUninitialized(worst_case) upfront and then emit without
// re-checking capacity per byte.
FORCEINLINE uint8* WritePtr(uint8* Cursor, uint64 Value)
{
	while (Value >= 0x80)
	{
		*Cursor++ = static_cast<uint8>(Value & 0x7F) | 0x80;
		Value >>= 7;
	}
	*Cursor++ = static_cast<uint8>(Value);
	return Cursor;
}

uint64 Read(const uint8*& Cursor, const uint8* End)
{
	uint64 Result = 0;
	uint32 Shift = 0;
	while (Cursor < End)
	{
		uint8 Byte = *Cursor++;
		Result |= static_cast<uint64>(Byte & 0x7F) << Shift;
		if ((Byte & 0x80) == 0)
		{
			return Result;
		}
		Shift += 7;
	}
	return Result;
}

} // namespace Varint

// ---------------------------------------------------------------------------
// Posting list encoding/decoding
//
// Format: [FreqGroup]*
// FreqGroup = { uint8 TF, varint Count, varint Delta0, varint Delta1, ... }
// DocIDs are sorted ascending within each TF group and delta-encoded.
// ---------------------------------------------------------------------------

namespace PostingCodec
{


/**
 * Sort a single TF bucket by DocID ascending. Below the threshold we defer to
 * Algo::Sort (introsort is cache-hot and branch-tolerant on small ranges). 
 * Above it we use an 8-bit LSB radix sort since it performs better on our randomly distributed hashes
 *
 * `Scratch` must point to at least `NumEntries` FPostingEntry slots disjoint from `Data`.
 */
static void SortBucketByDocID(FPostingEntry* Data, int32 NumEntries, FPostingEntry* Scratch)
{
	// Tuned for random-key workloads; below this, introsort tends to win on
	// small-N cache behavior. Revisit if bucket size distribution changes.
	constexpr int32 RadixThreshold = 128;

	if (NumEntries < RadixThreshold)
	{
		Algo::Sort(MakeArrayView(Data, NumEntries),
			[](const FPostingEntry& Lhs, const FPostingEntry& Rhs)
			{
				return Lhs.DocID < Rhs.DocID;
			});
		return;
	}

	FPostingEntry* Source = Data;
	FPostingEntry* Dest = Scratch;

	for (int32 Pass = 0; Pass < 8; ++Pass)
	{
		const int32 Shift = Pass * 8;

		uint32 PassCounts[256] = {};
		for (int32 EntryIndex = 0; EntryIndex < NumEntries; ++EntryIndex)
		{
			const uint32 Digit = static_cast<uint32>((static_cast<uint64>(Source[EntryIndex].DocID) >> Shift) & 0xFFu);
			++PassCounts[Digit];
		}

		uint32 Sum = 0;
		for (int32 DigitIndex = 0; DigitIndex < 256; ++DigitIndex)
		{
			const uint32 DigitCount = PassCounts[DigitIndex];
			PassCounts[DigitIndex] = Sum;
			Sum += DigitCount;
		}

		for (int32 EntryIndex = 0; EntryIndex < NumEntries; ++EntryIndex)
		{
			const uint32 Digit = static_cast<uint32>((static_cast<uint64>(Source[EntryIndex].DocID) >> Shift) & 0xFFu);
			Dest[PassCounts[Digit]++] = Source[EntryIndex];
		}

		Swap(Source, Dest);
	}
	// 8 is even, so Source ends up pointing back at Data
}

void Encode(TArray<uint8>& Buffer,
	TConstArrayView<FPostingEntry> Entries,
	TArray<FPostingEntry>& SortScratch,
	TArray<FPostingEntry>& RadixScratch)
{
	const int32 NumEntries = Entries.Num();
	if (NumEntries == 0)
	{
		return;
	}

	// Geometric-growth Reserve (Max(Max*2, Required)) so aggregate realloc stays O(NumEntries)
	const int64 Required = static_cast<int64>(Buffer.Num())
		+ static_cast<int64>(NumEntries) * 10
		+ 256 * 11;
	if (Buffer.Max() < Required)
	{
		Buffer.Reserve(FMath::Max<int64>(Buffer.Max() * 2, Required));
	}

	// Counting sort on TermFreq: TF is uint8 has a very low entropy in practice
	uint32 Counts[256] = {};
	for (int32 EntryIndex = 0; EntryIndex < NumEntries; ++EntryIndex)
	{
		++Counts[Entries[EntryIndex].TermFreq];
	}

	// Exclusive prefix sum -> bucket start offsets within SortScratch.
	uint32 Offsets[256];
	{
		uint32 Sum = 0;
		for (int32 TFValue = 0; TFValue < 256; ++TFValue)
		{
			Offsets[TFValue] = Sum;
			Sum += Counts[TFValue];
		}
	}

	// Partition entries into SortScratch by TF in a single linear pass.
	SortScratch.SetNumUninitialized(NumEntries, EAllowShrinking::No);
	RadixScratch.SetNumUninitialized(NumEntries, EAllowShrinking::No);
	uint32 PartitionCursor[256];
	FMemory::Memcpy(PartitionCursor, Offsets, sizeof(PartitionCursor));
	for (int32 EntryIndex = 0; EntryIndex < NumEntries; ++EntryIndex)
	{
		const FPostingEntry& Entry = Entries[EntryIndex];
		SortScratch[PartitionCursor[Entry.TermFreq]++] = Entry;
	}

	FPostingEntry* const RadixScratchBase = RadixScratch.GetData();

	// Emit each non-empty TF bucket in ascending TF order.
	for (int32 TFValue = 0; TFValue < 256; ++TFValue)
	{
		const uint32 GroupCount = Counts[TFValue];
		if (GroupCount == 0)
		{
			continue;
		}

		FPostingEntry* GroupBegin = SortScratch.GetData() + Offsets[TFValue];
		SortBucketByDocID(GroupBegin, static_cast<int32>(GroupCount), RadixScratchBase + Offsets[TFValue]);

		// Reserve worst-case bytes for this bucket (1 TF + 10 varint count + 10 per delta),
		// write via raw pointer to skip per-byte TArray::Add capacity checks, then trim to
		// actual bytes written. The outer Reserve above guarantees this AddUninitialized
		// never reallocates.
		const int32 WorstCaseBytes = 11 + 10 * static_cast<int32>(GroupCount);
		const int32 WriteStart = Buffer.AddUninitialized(WorstCaseBytes);
		uint8* const BufferBegin = Buffer.GetData();
		uint8* WriteCursor = BufferBegin + WriteStart;

		*WriteCursor++ = static_cast<uint8>(TFValue);
		WriteCursor = Varint::WritePtr(WriteCursor, static_cast<uint64>(GroupCount));

		int64 PrevDocID = 0;
		for (uint32 EntryIndex = 0; EntryIndex < GroupCount; ++EntryIndex)
		{
			const int64 Delta = GroupBegin[EntryIndex].DocID - PrevDocID;
			check(Delta >= 0);
			WriteCursor = Varint::WritePtr(WriteCursor, static_cast<uint64>(Delta));
			PrevDocID = GroupBegin[EntryIndex].DocID;
		}

		const int32 ActualBytes = static_cast<int32>(WriteCursor - (BufferBegin + WriteStart));
		Buffer.SetNum(WriteStart + ActualBytes, EAllowShrinking::No);
	}
}

/**
 * Templated decode that emits each entry to a callback. Lets callers consume
 * entries straight into a TSet (or any other sink) without the intermediate
 * TArray that the original Decode below allocates.
 *
 * @param Data    Pointer to the start of the encoded posting list bytes.
 * @param Length  Number of bytes in the encoded posting list.
 * @param Apply   Callable invoked with each decoded FPostingEntry by const-ref.
 */
template <typename FuncType>
void DecodeAndApply(const uint8* Data, uint32 Length, FuncType&& Apply)
{
	const uint8* Cursor = Data;
	const uint8* End = Data + Length;

	while (Cursor < End)
	{
		const uint8 TF = *Cursor++;
		const uint64 GroupCount = Varint::Read(Cursor, End);

		int64 PrevDocID = 0;
		for (uint64 EntryIndex = 0; EntryIndex < GroupCount && Cursor < End; ++EntryIndex)
		{
			const int64 Delta = static_cast<int64>(Varint::Read(Cursor, End));
			PrevDocID += Delta;

			FPostingEntry Entry;
			Entry.DocID = PrevDocID;
			Entry.TermFreq = TF;
			Apply(Entry);
		}
	}
}

/**
 * Fast pre-scan that returns the exact entry count without decoding deltas.
 * Walks group headers (TF byte + varint count) and skips past each delta varint
 * by following the continuation-bit chain.
 *
 * @param Data    Pointer to the start of the encoded posting list bytes.
 * @param Length  Number of bytes in the encoded posting list.
 * @return        Total number of entries that DecodeAndApply would emit.
 */
uint32 CountEntries(const uint8* Data, uint32 Length)
{
	const uint8* Cursor = Data;
	const uint8* End = Data + Length;
	uint32 Total = 0;

	while (Cursor < End)
	{
		++Cursor;  // TF byte
		if (Cursor >= End)
		{
			break;
		}
		const uint64 GroupCount = Varint::Read(Cursor, End);
		Total += static_cast<uint32>(GroupCount);

		// Skip past GroupCount delta varints (continuation bits set on all
		// bytes except the last of each varint).
		for (uint64 i = 0; i < GroupCount && Cursor < End; ++i)
		{
			while (Cursor < End && (*Cursor & 0x80))
			{
				++Cursor;
			}
			if (Cursor < End)
			{
				++Cursor;
			}
		}
	}
	return Total;
}

void Decode(const uint8* Data, uint32 Length, TArray<FPostingEntry>& OutEntries)
{
	DecodeAndApply(Data, Length, [&OutEntries](const FPostingEntry& Entry)
	{
		OutEntries.Add(Entry);
	});
}

} // namespace PostingCodec

/**
 * Combined posting list (compacted + staging) keyed by DocID. Backed by a flat
 * TArray of entries plus an FHashTable of DocID -> array index, so the scorer
 * gets O(1) per-token TF lookups without TSet's TSparseArray + TBitArray
 * overhead. Sized exactly once via Reserve, then write-once / read-many.
 */
class FMergedPostingSet
{
public:
	FMergedPostingSet() = default;

	/**
	 * Allocate exact capacity. One-shot — call on an empty set before any Add.
	 *
	 * @param Capacity  Total number of entries the set will hold. Sizes both
	 *                  the entries array and the hash table in a single shot
	 *                  so subsequent Add calls never grow either buffer.
	 */
	void Reserve(int32 Capacity)
	{
		Entries.Reserve(Capacity);
		if (Capacity > 0)
		{
			const uint32 HashSize = FMath::Max(32u, FMath::RoundUpToPowerOfTwo(static_cast<uint32>(Capacity)));
			HashTable.Clear(HashSize, static_cast<uint32>(Capacity));
		}
	}

	/** @return Number of entries currently in the set. */
	int32 Num() const { return Entries.Num(); }

	/**
	 * Append a posting entry unconditionally.
	 *
	 * @param Entry  Posting entry to append. Caller must ensure DocID
	 *               uniqueness across prior Add calls — use Contains() first
	 *               when merging sources that may collide.
	 */
	void Add(const FPostingEntry& Entry)
	{
		const uint32 Index = static_cast<uint32>(Entries.Num());
		Entries.Add(Entry);
		HashTable.Add(::GetTypeHash(Entry.DocID), Index);
	}

	/**
	 * Look up the posting entry for a document.
	 *
	 * @param DocID  Document ID to search for.
	 * @return       Pointer to the entry if present, nullptr otherwise. The
	 *               pointer is valid only until the next mutation of the set.
	 */
	const FPostingEntry* Find(int64 DocID) const
	{
		const uint32 Key = ::GetTypeHash(DocID);
		for (uint32 i = HashTable.First(Key); HashTable.IsValid(i); i = HashTable.Next(i))
		{
			if (Entries[i].DocID == DocID)
			{
				return &Entries[i];
			}
		}
		return nullptr;
	}

	/**
	 * @param DocID  Document ID to test for membership.
	 * @return       True if a posting entry with the given DocID is present.
	 */
	bool Contains(int64 DocID) const
	{
		return Find(DocID) != nullptr;
	}

	const FPostingEntry* begin() const
	{
		return Entries.GetData();
	}
	const FPostingEntry* end() const
	{
		return Entries.GetData() + Entries.Num();
	}

private:
	TArray<FPostingEntry> Entries;
	FHashTable HashTable;
};

// ---------------------------------------------------------------------------
// Tokenizer
// ---------------------------------------------------------------------------

namespace Tokenizer
{

uint32 HashToken(FStringView Token)
{
	return FCrc::StrCrc32(Token.GetData(), Token.Len());
}

const TSet<uint32>& GetStopWordHashes()
{
	static TSet<uint32> Hashes;
	if (Hashes.Num() == 0)
	{
		static const TCHAR* StopWords[] = {
			TEXT("the"), TEXT("is"), TEXT("in"), TEXT("on"), TEXT("at"),
			TEXT("to"), TEXT("a"), TEXT("an"), TEXT("and"), TEXT("or"),
			TEXT("of"), TEXT("for"), TEXT("with"), TEXT("this"), TEXT("that"),
			TEXT("by"), TEXT("be"), TEXT("was"), TEXT("were"), TEXT("it"),
			TEXT("as"), TEXT("from"), TEXT("but"), TEXT("if"), TEXT("then"),
			TEXT("out"), TEXT("so"), TEXT("such"), TEXT("when"), TEXT("which"),
			TEXT("while"), TEXT("are"), TEXT("has"), TEXT("have"), TEXT("had"),
			TEXT("not"), TEXT("no"), TEXT("can"), TEXT("will"), TEXT("do"),
		};
		for (const TCHAR* Word : StopWords)
		{
			Hashes.Add(HashToken(FStringView(Word)));
		}
	}
	return Hashes;
}

// Split a single part on CamelCase boundaries, output lowercase token hashes.
void SplitCamelCase(FStringView Part, TArray<uint32>& OutHashes, const TSet<uint32>& StopHashes)
{
	const int32 Len = Part.Len();
	if (Len == 0)
	{
		return;
	}

	const TCHAR* Data = Part.GetData();
	int32 Start = 0;

	auto EmitToken = [Data, &OutHashes, &StopHashes](int32 TokenStart, int32 TokenEnd)
	{
		if (TokenEnd <= TokenStart)
		{
			return;
		}
		// Build lowercase token and hash it
		FString Lower;
		Lower.Reserve(TokenEnd - TokenStart);
		for (int32 j = TokenStart; j < TokenEnd; ++j)
		{
			Lower.AppendChar(FChar::ToLower(Data[j]));
		}
		uint32 Hash = HashToken(Lower);
		if (!StopHashes.Contains(Hash))
		{
			OutHashes.Add(Hash);
		}
	};

	for (int32 i = 1; i <= Len; ++i)
	{
		bool bSplit = (i == Len);

		if (!bSplit)
		{
			const TCHAR Prev = Data[i - 1];
			const TCHAR Curr = Data[i];

			// Upper->Lower with multi-char prefix: "GRass" -> "G" | "Rass"
			if (FChar::IsUpper(Prev) && FChar::IsLower(Curr) && (i - Start) > 1)
			{
				EmitToken(Start, i - 1);
				Start = i - 1;
				continue;
			}
			// Lower->Upper: "grass" | "Texture"
			if (FChar::IsLower(Prev) && FChar::IsUpper(Curr))
			{
				bSplit = true;
			}
			// Letter<->Digit transitions
			if (FChar::IsDigit(Prev) != FChar::IsDigit(Curr))
			{
				bSplit = true;
			}
		}

		if (bSplit && i > Start)
		{
			EmitToken(Start, i);
			Start = i;
		}
	}
}

TArray<uint32> Tokenize(FStringView Text)
{
	TArray<uint32> TokenHashes;
	const TSet<uint32>& StopHashes = GetStopWordHashes();

	const int32 Len = Text.Len();
	const TCHAR* Data = Text.GetData();
	int32 Start = -1;

	// Phase 1: split on non-alphanumeric (underscore is separator)
	for (int32 i = 0; i <= Len; ++i)
	{
		bool bIsWordChar = (i < Len) && FChar::IsAlnum(Data[i]);

		if (bIsWordChar)
		{
			if (Start < 0)
			{
				Start = i;
			}
		}
		else if (Start >= 0)
		{
			// Phase 2: CamelCase split this part
			SplitCamelCase(FStringView(Data + Start, i - Start), TokenHashes, StopHashes);
			Start = -1;
		}
	}

	return TokenHashes;
}

} // namespace Tokenizer

// ---------------------------------------------------------------------------
// FBM25Index implementation
// ---------------------------------------------------------------------------

void FBM25Index::DecodePostingList(const uint8* Data, uint32 Length, TArray<FPostingEntry>& OutEntries)
{
	PostingCodec::Decode(Data, Length, OutEntries);
}

void FBM25Index::GetMergedPostingList(uint32 TokenHash, FMergedPostingSet& OutEntries) const
{
	// Staging is unique per (TokenHash, DocID) — Add/Remove purge old entries
	// before reinsert, so we append staging directly without dedup. Compacted
	// entries that overlap a staging DocID are dropped via Contains;
	const TArray<FPostingEntry>* Staging = StagingPostings.Find(TokenHash);
	const FPostingListRef* Ref = PostingIndex.Find(TokenHash);

	const int32 StagingNum = Staging ? Staging->Num() : 0;
	const uint32 CompactedCount = Ref
		? PostingCodec::CountEntries(PostingData.GetData() + Ref->Offset, Ref->Length)
		: 0;

	OutEntries.Reserve(StagingNum + static_cast<int32>(CompactedCount));

	if (Staging)
	{
		for (const FPostingEntry& E : *Staging)
		{
			OutEntries.Add(E);
		}
	}

	// Compacted entries — skip any DocID that has a staging override or is removed.
	// RemovedIDs only masks compacted entries; staging is purged eagerly.
	if (Ref)
	{
		const bool bHasStaging = StagingNum > 0;
		PostingCodec::DecodeAndApply(PostingData.GetData() + Ref->Offset, Ref->Length,
			[this, bHasStaging, &OutEntries](const FPostingEntry& E)
			{
				if (RemovedIDs.Contains(E.DocID))
				{
					return;
				}
				if (bHasStaging && OutEntries.Contains(E.DocID))
				{
					return;
				}
				OutEntries.Add(E);
			});
	}
}

void FBM25Index::Add(int64 ID, FStringView AssetPath, FStringView Caption, TConstArrayView<FString> Keywords)
{
	// Tokenize all fields
	TArray<uint32> AllTokens;
	AllTokens.Append(Tokenizer::Tokenize(AssetPath));
	AllTokens.Append(Tokenizer::Tokenize(Caption));
	for (const FString& Keyword : Keywords)
	{
		AllTokens.Append(Tokenizer::Tokenize(Keyword));
	}

	if (AllTokens.Num() == 0)
	{
		return;
	}

	// Build term frequency map
	TMap<uint32, uint8> TermFreqs;
	for (uint32 Hash : AllTokens)
	{
		uint8& Freq = TermFreqs.FindOrAdd(Hash, 0);
		if (Freq < 255)
		{
			Freq++;
		}
	}

	// If this document already exists, invalidate old entries
	if (DocLengths.Contains(ID))
	{
		if (uint16* OldLen = DocLengths.Find(ID))
		{
			TotalTokenCount -= *OldLen;
		}
		DocLengths.Remove(ID);

		// Mark removed so compacted entries are masked in GetMergedPostingList
		RemovedIDs.Add(ID);

		// Purge old staging entries for this DocID
		for (auto& [Hash, Entries] : StagingPostings)
		{
			Entries.RemoveAll([ID](const FPostingEntry& E) { return E.DocID == ID; });
		}
	}

	// Add to staging posting lists
	for (const auto& Pair : TermFreqs)
	{
		FPostingEntry Entry;
		Entry.DocID = ID;
		Entry.TermFreq = Pair.Value;
		StagingPostings.FindOrAdd(Pair.Key).Add(Entry);
	}

	const uint16 DocLen = static_cast<uint16>(FMath::Min(AllTokens.Num(), (int32)MAX_uint16));
	DocLengths.Add(ID, DocLen);
	TotalTokenCount += DocLen;
}

void FBM25Index::Remove(int64 ID)
{
	if (!DocLengths.Contains(ID))
	{
		return;
	}

	if (uint16* Len = DocLengths.Find(ID))
	{
		TotalTokenCount -= *Len;
	}
	DocLengths.Remove(ID);

	// Mask compacted entries
	RemovedIDs.Add(ID);

	// Purge staging entries eagerly
	for (auto& [Hash, Entries] : StagingPostings)
	{
		Entries.RemoveAll([ID](const FPostingEntry& E) { return E.DocID == ID; });
	}
}

void FBM25Index::Remove(TConstArrayView<int64> IDs)
{
	for (int64 ID : IDs)
	{
		Remove(ID);
	}
}

void FBM25Index::Add(TConstArrayView<int64> IDs, TConstArrayView<FString> Paths,
	TConstArrayView<FString> Captions, const TArray<TArray<FString>>& AllKeywords)
{
	check(IDs.Num() == Paths.Num() && IDs.Num() == Captions.Num() && IDs.Num() == AllKeywords.Num());

	if (IDs.Num() == 0)
	{
		return;
	}

	// Phase 1: Tokenize all documents, build per-doc term frequencies
	struct FDocData
	{
		TMap<uint32, uint8> TermFreqs;
		uint16 DocLen = 0;
	};
	TArray<FDocData> AllDocs;
	AllDocs.SetNum(IDs.Num());

	for (int32 i = 0; i < IDs.Num(); ++i)
	{
		TArray<uint32> AllTokens;
		AllTokens.Append(Tokenizer::Tokenize(Paths[i]));
		AllTokens.Append(Tokenizer::Tokenize(Captions[i]));
		for (const FString& Kw : AllKeywords[i])
		{
			AllTokens.Append(Tokenizer::Tokenize(Kw));
		}

		if (AllTokens.Num() == 0)
		{
			continue;
		}

		// Build term frequency map
		for (uint32 Hash : AllTokens)
		{
			uint8& Freq = AllDocs[i].TermFreqs.FindOrAdd(Hash, 0);
			if (Freq < 255)
			{
				Freq++;
			}
		}

		AllDocs[i].DocLen = static_cast<uint16>(FMath::Min(AllTokens.Num(), (int32)MAX_uint16));
	}

	// Phase 2: Invalidate existing docs — mask compacted entries and purge old staging
	TSet<int64> ReAddedIDs;
	for (int64 ID : IDs)
	{
		if (DocLengths.Contains(ID))
		{
			if (uint16* OldLen = DocLengths.Find(ID))
			{
				TotalTokenCount -= *OldLen;
			}
			DocLengths.Remove(ID);
			RemovedIDs.Add(ID);
			ReAddedIDs.Add(ID);
		}
	}

	// Purge old staging entries for re-added DocIDs in one pass
	if (ReAddedIDs.Num() > 0)
	{
		for (auto& [Hash, Entries] : StagingPostings)
		{
			Entries.RemoveAll([&ReAddedIDs](const FPostingEntry& E) { return ReAddedIDs.Contains(E.DocID); });
		}
	}

	// Phase 3: Collect all (token_hash -> entries) across all docs
	TMap<uint32, TArray<FPostingEntry>> BatchPostings;
	for (int32 i = 0; i < IDs.Num(); ++i)
	{
		for (const auto& [Hash, Freq] : AllDocs[i].TermFreqs)
		{
			FPostingEntry Entry;
			Entry.DocID = IDs[i];
			Entry.TermFreq = Freq;
			BatchPostings.FindOrAdd(Hash).Add(Entry);
		}
	}

	// Phase 4: Batch-insert into staging posting lists
	for (auto& [Hash, Entries] : BatchPostings)
	{
		StagingPostings.FindOrAdd(Hash).Append(MoveTemp(Entries));
	}

	// Phase 5: Update doc metadata
	for (int32 i = 0; i < IDs.Num(); ++i)
	{
		if (AllDocs[i].DocLen > 0 || AllDocs[i].TermFreqs.Num() > 0)
		{
			DocLengths.Add(IDs[i], AllDocs[i].DocLen);
			TotalTokenCount += AllDocs[i].DocLen;
		}
	}
}

void FBM25Index::Search(
	FStringView Query,
	int32 K,
	const TSharedRef<const TArray<int64>>& IDFilter,
	TArray<uint32>& Scratch,
	FGraphEventRef IndexReadCompleteEvent,
	int32 MinShouldMatchPercent,
	TFunction<void(TArray<FBM25Result>&&)> Continuation) const
{
	TArray<uint32> QueryTokenHashes = Tokenizer::Tokenize(Query);
	if (QueryTokenHashes.Num() == 0)
	{
		if (IndexReadCompleteEvent.IsValid())
		{
			IndexReadCompleteEvent->DispatchSubsequents();
		}
		Continuation(TArray<FBM25Result>{});
		return;
	}

	const int32 NumDocs = DocLengths.Num();
	if (NumDocs == 0)
	{
		if (IndexReadCompleteEvent.IsValid())
		{
			IndexReadCompleteEvent->DispatchSubsequents();
		}
		Continuation(TArray<FBM25Result>{});
		return;
	}

	// Heap-allocated state for the deferred body.
	struct FSearchTaskState
	{
		const FBM25Index* Index = nullptr;
		TArray<uint32> QueryTokenHashes;
		TSharedPtr<const TArray<int64>, ESPMode::ThreadSafe> IDFilter;
		TFunction<void(TArray<FBM25Result>&&)> Continuation;
		TArray<uint32>* Scratch = nullptr;
		FGraphEventRef IndexReadCompleteEvent;
		int32 K = 0;
		int32 NumDocs = 0;
		int32 MinShouldMatchPercent = 0;
	};
	using FSharedState = TSharedPtr<FSearchTaskState, ESPMode::ThreadSafe>;

	FSharedState State = MakeShared<FSearchTaskState, ESPMode::ThreadSafe>();
	State->Index = this;
	State->QueryTokenHashes = MoveTemp(QueryTokenHashes);
	State->IDFilter = IDFilter.ToSharedPtr();
	State->Continuation = MoveTemp(Continuation);
	State->Scratch = &Scratch;
	State->IndexReadCompleteEvent = MoveTemp(IndexReadCompleteEvent);
	State->K = K;
	State->NumDocs = NumDocs;
	State->MinShouldMatchPercent = FMath::Clamp(MinShouldMatchPercent, 0, 100);

	auto SearchBody = [State]()
	{
		const FBM25Index* const This = State->Index;
		const int32 NumDocs = State->NumDocs;
		const int32 K = State->K;
		const TArray<int64>& IDFilter = *State->IDFilter;
		TArray<uint32>& Scratch = *State->Scratch;
		const TArray<uint32>& QueryTokenHashes = State->QueryTokenHashes;

		// Build merged posting lists per query token. Two phases:
		//  1. Pre-create one TMap slot per unique token 
		//  2. ParallelFor over snapshot pointers to fill each slot. Per-token merge
		//     work varies dramatically (a posting list can be tens of thousands of
		//     compacted entries or empty), so Unbalanced/work-stealing keeps cores
		//     busy and avoids stragglers
		TMap<uint32, FMergedPostingSet> MergedLists;
		MergedLists.Reserve(QueryTokenHashes.Num());
		for (uint32 Hash : QueryTokenHashes)
		{
			MergedLists.FindOrAdd(Hash);
		}

		struct FMergeWork
		{
			uint32 Hash;
			FMergedPostingSet* Merged;
		};
		TArray<FMergeWork, TInlineAllocator<8>> MergeWorkItems;
		MergeWorkItems.Reserve(MergedLists.Num());
		for (TPair<uint32, FMergedPostingSet>& Pair : MergedLists)
		{
			MergeWorkItems.Add({Pair.Key, &Pair.Value});
		}

		ParallelFor(
			TEXT("FBM25Index::Search Merge"),
			MergeWorkItems.Num(),
			/*MinBatchSize*/ 1,
			[This, &MergeWorkItems](int32 Idx)
			{
				const FMergeWork& Work = MergeWorkItems[Idx];
				This->GetMergedPostingList(Work.Hash, *Work.Merged);
			},
			Private::GetSearchParallelForFlags(EParallelForFlags::Unbalanced));

		// Hoist per-token constants out of the per-doc loop. Tokens with no merged
		// entries are dropped here so the inner loop only iterates productive tokens.
		// Iterating MergedLists (deduped) instead of QueryTokenHashes also avoids
		// double-counting IDF when a query repeats a term (e.g. "cat cat").
		struct FTokenContext
		{
			const FMergedPostingSet* Merged;
			float IDFScore;
		};
		TArray<FTokenContext, TInlineAllocator<8>> TokenCtx;
		TokenCtx.Reserve(MergedLists.Num());

		const float N = static_cast<float>(NumDocs);
		const float AvgDL = static_cast<float>(This->TotalTokenCount) / N;
		for (const TPair<uint32, FMergedPostingSet>& Pair : MergedLists)
		{
			const FMergedPostingSet& Merged = Pair.Value;
			if (Merged.Num() == 0)
			{
				continue;
			}
			const float Df = static_cast<float>(Merged.Num());
			const float IDFScore = FMath::Loge((N - Df + 0.5f) / (Df + 0.5f) + 1.0f);
			TokenCtx.Add({&Merged, IDFScore});
		}
		if (TokenCtx.Num() == 0)
		{
			// Merge ParallelFor finished but every token's posting list is empty.
			if (State->IndexReadCompleteEvent.IsValid())
			{
				State->IndexReadCompleteEvent->DispatchSubsequents();
			}
			State->Continuation(TArray<FBM25Result>{});
			return;
		}

		// Min-should-match: require docs to hit at least this many of the productive
		// query tokens. Denominator is TokenCtx.Num() (terms present in the corpus),
		// not the raw query length, so a typo / unindexed word doesn't suppress all
		// results.
		const int32 RequiredMatches = (State->MinShouldMatchPercent <= 0)
			? 1
			: FMath::Max(1, FMath::CeilToInt(TokenCtx.Num() * (State->MinShouldMatchPercent / 100.0f)));

		// Per-doc scorer.
		struct FTokenHit
		{
			float TermFreq;
			float IDFScore;
		};
		using FHitsScratch = TArray<FTokenHit, TInlineAllocator<8>>;
		auto ScoreOneInto = [&](int64 DocID, FHitsScratch& HitsScratch, float& OutScore) -> bool
		{
			HitsScratch.Reset();
			for (const FTokenContext& Ctx : TokenCtx)
			{
				if (const FPostingEntry* E = Ctx.Merged->Find(DocID))
				{
					if (E->TermFreq > 0)
					{
						HitsScratch.Add({static_cast<float>(E->TermFreq), Ctx.IDFScore});
					}
				}
			}
			if (HitsScratch.Num() < RequiredMatches)
			{
				return false;
			}

			const uint16* DocLenPtr = This->DocLengths.Find(DocID);
			if (!DocLenPtr)
			{
				return false;
			}
			const float DL = static_cast<float>(*DocLenPtr);
			// DL-dependent denominator term — same for every token of this doc.
			const float DenomDLPart = K1 * (1.0f - B + B * (DL / AvgDL));

			float Score = 0.0f;
			for (const FTokenHit& Hit : HitsScratch)
			{
				const float Numerator = Hit.IDFScore * Hit.TermFreq * (K1 + 1.0f);
				const float Denominator = Hit.TermFreq + DenomDLPart;
				Score += Numerator / Denominator;
			}
			OutScore = Score;
			return true;
		};

		// Choose the doc-id source for scoring:
		//  - Filtered: IDFilter is the authoritative set (callers pass unique IDs).
		//  - Unfiltered: candidates are the union of DocIDs across merged posting lists.
		// Both paths feed the same parallel scoring loop below.
		TConstArrayView<int64> DocIDsToScore;
		TArray<int64> UnfilteredCandidates;
		if (IDFilter.Num() > 0)
		{
			DocIDsToScore = IDFilter;
		}
		else
		{
			TSet<int64> CandidatesSet;
			for (const FTokenContext& Ctx : TokenCtx)
			{
				for (const FPostingEntry& Entry : *Ctx.Merged)
				{
					CandidatesSet.Add(Entry.DocID);
				}
			}
			UnfilteredCandidates = CandidatesSet.Array();
			DocIDsToScore = UnfilteredCandidates;
		}

		// Parallel per-doc scoring. Workers read TokenCtx and DocLengths only
		// (no shared mutable state), accumulate into per-task buffers, and we
		// concatenate at the end.
		constexpr int32 ScoreMinBatchSize = 8192;
		struct FScoreTaskContext
		{
			TArray<FBM25Result> Results;
			FHitsScratch HitsScratch;
		};
		TArray<FScoreTaskContext> TaskContexts;
		ParallelForWithTaskContext(
			TEXT("FBM25Index::Search Score"),
			TaskContexts,
			DocIDsToScore.Num(),
			ScoreMinBatchSize,
			[&](FScoreTaskContext& Ctx, int32 Idx)
			{
				const int64 DocID = DocIDsToScore[Idx];
				float Score = 0.0f;
				if (ScoreOneInto(DocID, Ctx.HitsScratch, Score) && Score > 0.0f)
				{
					FBM25Result Result;
					Result.ID = DocID;
					Result.Score = Score;
					Ctx.Results.Add(Result);
				}
			},
			Private::GetSearchParallelForFlags());

		if (State->IndexReadCompleteEvent.IsValid())
		{
			State->IndexReadCompleteEvent->DispatchSubsequents();
		}

		TArray<FBM25Result> Results;
		int32 TotalResults = 0;
		for (const FScoreTaskContext& Ctx : TaskContexts)
		{
			TotalResults += Ctx.Results.Num();
		}
		Results.Reserve(TotalResults);
		for (FScoreTaskContext& Ctx : TaskContexts)
		{
			Results.Append(MoveTemp(Ctx.Results));
		}

		// Sort descending by score.
		Private::RadixSort(Results, Scratch, [](const FBM25Result& R) { return R.Score; }, /*bDescending*/ true);

		if (Results.Num() > K)
		{
			Results.SetNum(K);
		}

		State->Continuation(MoveTemp(Results));
	};

	Private::RunOrDispatchIndexWorker(MoveTemp(SearchBody));
}

int64 FBM25Index::GetCount() const
{
	return DocLengths.Num();
}

int64 FBM25Index::EstimateMemoryBytes() const
{
	int64 Bytes = 0;

	// Compacted posting data
	Bytes += PostingData.GetAllocatedSize();
	Bytes += PostingIndex.GetAllocatedSize();

	// Staging posting lists
	Bytes += StagingPostings.GetAllocatedSize();
	for (const auto& [Hash, Postings] : StagingPostings)
	{
		Bytes += Postings.GetAllocatedSize();
	}

	// RemovedIDs
	Bytes += RemovedIDs.GetAllocatedSize();

	// DocLengths
	Bytes += DocLengths.GetAllocatedSize();

	return Bytes;
}

void FBM25Index::Clear()
{
	PostingData.Empty();
	PostingIndex.Empty();
	StagingPostings.Empty();
	RemovedIDs.Empty();
	DocLengths.Empty();
	TotalTokenCount = 0;
}

bool FBM25Index::Contains(int64 ID) const
{
	return DocLengths.Contains(ID);
}

// ---------------------------------------------------------------------------
// Compaction
// ---------------------------------------------------------------------------

void FBM25Index::Compact()
{
	TArray<uint8> NewPostingData;
	TMap<uint32, FPostingListRef> NewPostingIndex;

	/**
	 * Size the output buffer against the actual work : preserved compacted bytes
	 * + ~10 bytes per staging entry (varint worst case) + ~16 bytes of per-token
	 * header overhead.
	 */
	int64 TotalStagingEntries = 0;
	for (const TPair<uint32, TArray<FPostingEntry>>& StagingPair : StagingPostings)
	{
		TotalStagingEntries += StagingPair.Value.Num();
	}
	const int64 DesiredBytes = static_cast<int64>(PostingData.Num())
		+ TotalStagingEntries * 10
		+ static_cast<int64>(StagingPostings.Num()) * 16;

	NewPostingData.Reserve(DesiredBytes);

	const bool bHasRemovals = RemovedIDs.Num() > 0;

	// One work item per token. Per-item Entries and Output are owned by the struct
	// so the parallel encode phase has no shared mutable state. The fast-copy path
	// is handled in phase 3 without an encode.
	struct FEncodeWork
	{
		uint32 Hash = 0;
		TArray<FPostingEntry> Entries;
		TArray<uint8> Output;
		uint32 FastCopyOffset = 0;
		uint32 FastCopyLength = 0;
		bool bFastCopy = false;
	};

	TArray<FEncodeWork> WorkItems;
	WorkItems.Reserve(PostingIndex.Num() + StagingPostings.Num());

	// Phase 1 (serial): for each token, decode its compacted posting list, drop entries overridden by staging (via the StagingDocIDs scratch set) or listed in RemovedIDs, then append the staging entries into Work.Entries.
	TArray<FPostingEntry> Decoded;
	TSet<int64> StagingDocIDs;

	for (const TPair<uint32, FPostingListRef>& PostingPair : PostingIndex)
	{
		const uint32 Hash = PostingPair.Key;
		const FPostingListRef& Ref = PostingPair.Value;

		const TArray<FPostingEntry>* Staging = StagingPostings.Find(Hash);

		FEncodeWork& Work = WorkItems.AddDefaulted_GetRef();
		Work.Hash = Hash;

		// Fast path: nothing touches this list, schedule a byte-copy for phase 3.
		if (Staging == nullptr && !bHasRemovals)
		{
			Work.bFastCopy = true;
			Work.FastCopyOffset = Ref.Offset;
			Work.FastCopyLength = Ref.Length;
			continue;
		}

		Decoded.Reset();
		StagingDocIDs.Reset();

		if (Staging != nullptr)
		{
			for (const FPostingEntry& E : *Staging)
			{
				StagingDocIDs.Add(E.DocID);
			}
		}

		DecodePostingList(PostingData.GetData() + Ref.Offset, Ref.Length, Decoded);
		const bool bCheckStaging = StagingDocIDs.Num() > 0;
		Work.Entries.Reserve(Decoded.Num() + (Staging ? Staging->Num() : 0));
		for (const FPostingEntry& E : Decoded)
		{
			if (bCheckStaging && StagingDocIDs.Contains(E.DocID))
			{
				continue;
			}
			if (bHasRemovals && RemovedIDs.Contains(E.DocID))
			{
				continue;
			}
			Work.Entries.Add(E);
		}

		if (Staging != nullptr)
		{
			Work.Entries.Append(*Staging);
		}
	}

	// Staging - only tokens(no compacted counterpart).
	for (TPair<uint32, TArray<FPostingEntry>>& StagingPair : StagingPostings)
	{
		const uint32 Hash = StagingPair.Key;
		TArray<FPostingEntry>& Staging = StagingPair.Value;

		if (Staging.Num() == 0 || PostingIndex.Contains(Hash))
		{
			continue;
		}

		FEncodeWork& Work = WorkItems.AddDefaulted_GetRef();
		Work.Hash = Hash;

		// MoveTemp the TArray out of the map. StagingPostings is Reset() at the end of Compact
		Work.Entries = MoveTemp(Staging);
	}

	// Phase 2 (parallel): Encodings, each worker gets their own scratch buffer to save time, Unbalanced to allow work stealing.
	// It might be slower in some small data set but it is worth it for the large wall time gains in larger datasets
	struct FEncodeTaskContext
	{
		TArray<FPostingEntry> SortScratch;
		TArray<FPostingEntry> RadixScratch;
	};
	TArray<FEncodeTaskContext> TaskContexts;

	ParallelForWithTaskContext(TaskContexts, WorkItems.Num(),
		[&WorkItems](FEncodeTaskContext& Context, int32 Idx)
		{
			FEncodeWork& Work = WorkItems[Idx];
			if (Work.bFastCopy || Work.Entries.Num() == 0)
			{
				return;
			}
			PostingCodec::Encode(Work.Output, Work.Entries, Context.SortScratch, Context.RadixScratch);
		},
		EParallelForFlags::Unbalanced);

	// Phase 3 (serial): concatenate per-token outputs into NewPostingData and build the final index. 
	NewPostingIndex.Reserve(WorkItems.Num());
	for (const FEncodeWork& Work : WorkItems)
	{
		FPostingListRef NewRef;
		NewRef.Offset = static_cast<uint32>(NewPostingData.Num());

		if (Work.bFastCopy)
		{
			NewRef.Length = Work.FastCopyLength;
			NewPostingData.Append(PostingData.GetData() + Work.FastCopyOffset, Work.FastCopyLength);
		}
		else if (Work.Output.Num() > 0)
		{
			NewRef.Length = static_cast<uint32>(Work.Output.Num());
			NewPostingData.Append(Work.Output);
		}
		else
		{
			// Removed-only token — nothing left to emit.
			continue;
		}

		NewPostingIndex.Add(Work.Hash, NewRef);
	}

	PostingData = MoveTemp(NewPostingData);
	PostingIndex = MoveTemp(NewPostingIndex);
	StagingPostings.Reset();
	RemovedIDs.Reset();
}

bool FBM25Index::IsCompacted() const
{
	return StagingPostings.Num() == 0 && RemovedIDs.Num() == 0;
}

// ---------------------------------------------------------------------------
// Serialization
// ---------------------------------------------------------------------------

void FBM25Index::Serialize(FArchive& Ar)
{
	// Ensure data is compacted before writing
	Compact();

	uint32 WriteMagic = FBM25IndexHeader::Magic;
	uint32 WriteVersion = FBM25IndexHeader::Version;
	Ar << WriteMagic << WriteVersion;

	// Consolidated posting buffer
	Ar << PostingData;

	// Posting index
	int32 NumEntries = PostingIndex.Num();
	Ar << NumEntries;
	for (auto& [Hash, Ref] : PostingIndex)
	{
		uint32 MutableHash = Hash;
		Ar << MutableHash << Ref.Offset << Ref.Length;
	}

	Ar << DocLengths;
	Ar << TotalTokenCount;
}

TUniquePtr<FBM25Index> FBM25Index::Deserialize(FArchive& Ar)
{
	uint32 Magic = 0;
	uint32 Version = 0;
	Ar << Magic << Version;
	if (Magic != FBM25IndexHeader::Magic || Version != FBM25IndexHeader::Version)
	{
		return nullptr;
	}

	TUniquePtr<FBM25Index> Index = MakeUnique<FBM25Index>();

	Ar << Index->PostingData;

	int32 NumEntries = 0;
	Ar << NumEntries;
	Index->PostingIndex.Reserve(NumEntries);
	for (int32 i = 0; i < NumEntries; ++i)
	{
		uint32 Hash = 0;
		FPostingListRef Ref;
		Ar << Hash << Ref.Offset << Ref.Length;
		Index->PostingIndex.Add(Hash, Ref);
	}

	Ar << Index->DocLengths;
	Ar << Index->TotalTokenCount;

	if (Ar.IsError())
	{
		return nullptr;
	}
	return Index;
}

} // namespace UE::SemanticSearch
