// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "Serialization/Archive.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

namespace UE::SemanticSearch
{

struct FBM25Result
{
	int64 ID = -1;
	float Score = 0.0f;
};

struct FPostingEntry
{
	int64 DocID;
	uint8 TermFreq;  // capped at 255

	friend FArchive& operator<<(FArchive& Ar, FPostingEntry& Entry)
	{
		return Ar << Entry.DocID << Entry.TermFreq;
	}
};

class FMergedPostingSet;

/**
 * Memory-efficient BM25 inverted index for text search over asset captions,
 * keywords, and paths.
 */
class FBM25Index
{
public:
	FBM25Index() = default;

	/** Add a document to the index. Tokenizes path, caption, and keywords. */
	void Add(int64 ID, FStringView AssetPath, FStringView Caption, TConstArrayView<FString> Keywords);

	/** Remove a document from the index. */
	void Remove(int64 ID);

	/** Batch remove multiple documents. Single sweep through affected posting lists. */
	void Remove(TConstArrayView<int64> IDs);

	/** Batch add multiple documents. Groups posting list insertions for efficiency. */
	void Add(TConstArrayView<int64> IDs, TConstArrayView<FString> Paths,
		TConstArrayView<FString> Captions, const TArray<TArray<FString>>& AllKeywords);

	/**
	 * Search for documents matching the query. Continuation receives the top K
	 * results sorted by score descending. The continuation is owned (by-value
	 * TFunction) so the implementation may move it into a deferred task and
	 * invoke it asynchronously off the calling thread.
	 *
	 * @param Query                    Text query to tokenize and score against the index.
	 * @param K                        Maximum number of results to return.
	 * @param IDFilter                 Shared, immutable list of IDs to restrict the search
	 *                                 to. Empty array means "no filter". Passed by
	 *                                 TSharedRef so the deferred task body can keep
	 *                                 ownership without copying — IDFilter can be large
	 *                                 (the whole post-filter set of a previous query,
	 *                                 hundreds of thousands of int64s) and the deep copy
	 *                                 used to dominate setup time.
	 * @param Scratch                  Caller-owned scratch buffer reused across calls.
	 * @param IndexReadCompleteEvent   Optional event fired after the index-touching
	 *                                 phase finishes (before sort + continuation),
	 *                                 letting waiters release index mutations early.
	 *                                 Pass an unset FGraphEventRef when not needed.
	 * @param MinShouldMatchPercent    Minimum percentage of unique productive query
	 *                                 tokens a document must match to be eligible.
	 *                                 0 = pure OR (any single term match returns the
	 *                                 doc). 100 = strict AND. Required match count
	 *                                 is computed via ceiling against the number of
	 *                                 query tokens that appear in at least one doc,
	 *                                 e.g. 75% of 4 terms = 3.
	 * @param Continuation             Invoked exactly once with the result array.
	 */
	void Search(
		FStringView Query,
		int32 K,
		const TSharedRef<const TArray<int64>>& IDFilter,
		TArray<uint32>& Scratch,
		FGraphEventRef IndexReadCompleteEvent,
		int32 MinShouldMatchPercent,
		TFunction<void(TArray<FBM25Result>&&)> Continuation) const;

	/** Remove all documents from the index. */
	void Clear();

	/** Number of documents in the index. */
	int64 GetCount() const;

	/** Check if a document is already indexed. */
	bool Contains(int64 ID) const;

	/** Serialize the index to an archive. Compacts before writing. */
	void Serialize(FArchive& Ar);

	/** Deserialize a BM25 index from an archive. Returns null on version mismatch or corrupt data. */
	static TUniquePtr<FBM25Index> Deserialize(FArchive& Ar);

	/** Estimate the memory footprint of this index in bytes. */
	int64 EstimateMemoryBytes() const;

	/**
	 * Compact the index: encode all staging posting lists into a consolidated
	 * byte buffer with frequency grouping + delta + varint encoding.
	 * Purges RemovedIDs. Call after bulk indexing completes.
	 */
	void Compact();

	/** True if the index has been compacted and staging is empty. */
	bool IsCompacted() const;

private:
	struct FPostingListRef
	{
		uint32 Offset;  // byte offset into PostingData
		uint32 Length;   // byte length of encoded posting list
	};

	/** Decode a compacted posting list, appending entries to OutEntries. */
	static void DecodePostingList(const uint8* Data, uint32 Length, TArray<FPostingEntry>& OutEntries);

	/** Get the combined (compacted + staging) posting list for a token hash, keyed by DocID. */
	void GetMergedPostingList(uint32 TokenHash, FMergedPostingSet& OutEntries) const;

	static constexpr float K1 = 1.5f;
	static constexpr float B = 0.75f;

	// --- Compacted data (read-only after Compact()) ---
	TArray<uint8> PostingData;
	TMap<uint32, FPostingListRef> PostingIndex;

	// --- Staging (mutable, used during indexing and for post-compact edits) ---
	TMap<uint32, TArray<FPostingEntry>> StagingPostings;

	// --- Lazy-delete set (filtered during search, purged on Compact) ---
	TSet<int64> RemovedIDs;

	// Per-document: doc length (token count)
	TMap<int64, uint16> DocLengths;

	// Global stats
	int64 TotalTokenCount = 0;

};

} // namespace UE::SemanticSearch
