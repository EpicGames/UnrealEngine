// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlatVectorIndex.h"

THIRD_PARTY_INCLUDES_START
#include "faiss/index_io.h"
#include "faiss/impl/io.h"
THIRD_PARTY_INCLUDES_END

namespace UE::SemanticSearch
{

FFlatVectorIndex::FFlatVectorIndex(int32 InDimension)
	: Dimension(InDimension)
{
	faiss::IndexFlatL2* FlatIndex = new faiss::IndexFlatL2(Dimension);
	IndexWithIDs = new faiss::IndexIDMap2(FlatIndex);
}

FFlatVectorIndex::FFlatVectorIndex(int32 InDimension, faiss::IndexIDMap2* InIndex)
	: Dimension(InDimension)
	, IndexWithIDs(InIndex)
{
}

FFlatVectorIndex::~FFlatVectorIndex()
{
	delete IndexWithIDs;
}

void FFlatVectorIndex::Train(TConstArrayView<float> Vectors, int64 NumVectors)
{
	// Flat index does not need training.
}

void FFlatVectorIndex::Add(TConstArrayView<int64> IDs, TConstArrayView<float> Vectors)
{
	const int64 Count = IDs.Num();
	check(Vectors.Num() == Count * Dimension);
	static_assert(sizeof(faiss::idx_t) == sizeof(int64), "FAISS idx_t size mismatch");

	IndexWithIDs->add_with_ids(
		Count,
		Vectors.GetData(),
		reinterpret_cast<const faiss::idx_t*>(IDs.GetData()));
}

void FFlatVectorIndex::Remove(TConstArrayView<int64> IDs)
{
	if (IDs.Num() == 0)
	{
		return;
	}
	static_assert(sizeof(faiss::idx_t) == sizeof(int64), "FAISS idx_t size mismatch");
	faiss::IDSelectorBatch Selector(IDs.Num(), reinterpret_cast<const faiss::idx_t*>(IDs.GetData()));

	IndexWithIDs->remove_ids(Selector);
}

void FFlatVectorIndex::Search(
	TConstArrayView<float> QueryVector,
	int32 K,
	const TSharedRef<const TArray<int64>>& IDFilterRef,
	float DistanceCutoff,
	TArray<uint32>& /*Scratch*/,
	FGraphEventRef IndexReadCompleteEvent,
	TFunction<void(TArray<FSearchResult>&&)> Continuation) const
{
	const TArray<int64>& IDFilter = *IDFilterRef;
	check(QueryVector.Num() == Dimension);

	// Helper for the index-read-complete signal. The flat path is fully
	// synchronous, so the event fires inline before the function returns.
	auto SignalIndexReadComplete = [&IndexReadCompleteEvent]
	{
		if (IndexReadCompleteEvent.IsValid())
		{
			IndexReadCompleteEvent->DispatchSubsequents();
		}
	};

	// Clamp K to avoid requesting more results than possible
	int32 EffectiveK = K;
	if (IDFilter.Num() > 0)
	{
		// When filtering, we can't return more results than the filter size (or number in index)
		EffectiveK = FMath::Min(FMath::Min(K, IDFilter.Num()),static_cast<int32>(IndexWithIDs->ntotal));
	}
	else
	{
		// Without a filter, clamp to the total number of indexed vectors
		EffectiveK = FMath::Min(K, static_cast<int32>(IndexWithIDs->ntotal));
	}

	// Early out if no results are possible
	if (EffectiveK <= 0)
	{
		SignalIndexReadComplete();
		Continuation(TArray<FSearchResult>{});
		return;
	}

	TArray<float> Distances;
	TArray<faiss::idx_t> Labels;
	Distances.SetNumUninitialized(EffectiveK);
	Labels.SetNumUninitialized(EffectiveK);

	if (IDFilter.Num() > 0)
	{
		static_assert(sizeof(faiss::idx_t) == sizeof(int64), "FAISS idx_t size mismatch");
		faiss::IDSelectorBatch Selector(IDFilter.Num(), reinterpret_cast<const faiss::idx_t*>(IDFilter.GetData()));
		faiss::SearchParameters Params;
		Params.sel = &Selector;
		IndexWithIDs->search(1, QueryVector.GetData(), EffectiveK, Distances.GetData(), Labels.GetData(), &Params);
	}
	else
	{
		IndexWithIDs->search(1, QueryVector.GetData(), EffectiveK, Distances.GetData(), Labels.GetData());
	}

	SignalIndexReadComplete();

	// FAISS returns results sorted ascending by distance. Once we hit one above
	// the cutoff, all subsequent ones are too — break out instead of continuing
	// to scan and discard.
	TArray<FSearchResult> Results;
	Results.Reserve(EffectiveK);
	for (int32 i = 0; i < EffectiveK; ++i)
	{
		if (Labels[i] < 0)
		{
			continue;
		}
		if (Distances[i] >= DistanceCutoff)
		{
			break;
		}
		FSearchResult Result;
		Result.ID = static_cast<int64>(Labels[i]);
		Result.Distance = Distances[i];
		Results.Add(Result);
	}
	Continuation(MoveTemp(Results));
}

int64 FFlatVectorIndex::GetCount() const
{

	return IndexWithIDs->ntotal;
}

bool FFlatVectorIndex::Contains(int64 ID) const
{

	return IndexWithIDs->rev_map.find(ID) != IndexWithIDs->rev_map.end();
}

bool FFlatVectorIndex::TryGetEmbedding(int64 ID, TArray<float>& OutEmbedding) const
{

	if (IndexWithIDs->rev_map.find(ID) == IndexWithIDs->rev_map.end())
	{
		return false;
	}

	OutEmbedding.SetNumUninitialized(Dimension);
	IndexWithIDs->reconstruct(static_cast<faiss::idx_t>(ID), OutEmbedding.GetData());
	return true;
}

int64 FFlatVectorIndex::EstimateMemoryBytes() const
{

	const int64 Count = IndexWithIDs->ntotal;
	// Raw vectors: Count * Dimension * sizeof(float)
	// ID overhead: ~24 bytes/entry for id_map + rev_map hash entry
	return Count * Dimension * sizeof(float) + Count * 24;
}

TArray<uint8> FFlatVectorIndex::Serialize() const
{

	faiss::VectorIOWriter Writer;
	faiss::write_index(IndexWithIDs, &Writer);

	TArray<uint8> Result;
	Result.Append(
		reinterpret_cast<const uint8*>(Writer.data.data()),
		static_cast<int32>(Writer.data.size()));
	return Result;
}

TUniquePtr<FFlatVectorIndex> FFlatVectorIndex::Deserialize(TConstArrayView<uint8> Data, int32 ExpectedDimension)
{
	faiss::VectorIOReader Reader;
	Reader.data.assign(Data.GetData(), Data.GetData() + Data.Num());

	faiss::Index* LoadedIndex = faiss::read_index(&Reader);
	if (!LoadedIndex || LoadedIndex->d != ExpectedDimension)
	{
		delete LoadedIndex;
		return nullptr;
	}

	faiss::IndexIDMap2* IDMapIndex = static_cast<faiss::IndexIDMap2*>(LoadedIndex);
	return TUniquePtr<FFlatVectorIndex>(new FFlatVectorIndex(ExpectedDimension, IDMapIndex));
}

} // namespace UE::SemanticSearch
