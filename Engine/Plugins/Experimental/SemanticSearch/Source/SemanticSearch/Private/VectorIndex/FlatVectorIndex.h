// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IVectorIndex.h"
#include "Templates/UniquePtr.h"

THIRD_PARTY_INCLUDES_START
#include "faiss/IndexFlat.h"
#include "faiss/IndexIDMap.h"
THIRD_PARTY_INCLUDES_END

namespace UE::SemanticSearch
{

/**
 * Flat (brute-force) vector index using L2 distance.
 *
 * Stores raw vectors without compression. Exact search results.
 * No training required. Good for small datasets or ground-truth comparison.
 */
class FFlatVectorIndex : public IVectorIndex
{
public:
	explicit FFlatVectorIndex(int32 InDimension);
	virtual ~FFlatVectorIndex() override;

	FFlatVectorIndex(const FFlatVectorIndex&) = delete;
	FFlatVectorIndex& operator=(const FFlatVectorIndex&) = delete;

	// IVectorIndex batch overrides
	virtual void Train(TConstArrayView<float> Vectors, int64 NumVectors) override;
	virtual bool IsTrained() const override { return true; }
	virtual void Add(TConstArrayView<int64> IDs, TConstArrayView<float> Vectors) override;
	virtual void Remove(TConstArrayView<int64> IDs) override;
	virtual void Search(
		TConstArrayView<float> QueryVector,
		int32 K,
		const TSharedRef<const TArray<int64>>& IDFilter,
		float DistanceCutoff,
		TArray<uint32>& Scratch,
		FGraphEventRef IndexReadCompleteEvent,
		TFunction<void(TArray<FSearchResult>&&)> Continuation) const override;
	virtual int64 GetCount() const override;
	virtual int32 GetDimension() const override { return Dimension; }
	virtual TArray<uint8> Serialize() const override;
	virtual bool Contains(int64 ID) const override;
	virtual bool TryGetEmbedding(int64 ID, TArray<float>& OutEmbedding) const override;
	virtual int64 EstimateMemoryBytes() const override;
	/** Deserialize a full flat index. */
	static TUniquePtr<FFlatVectorIndex> Deserialize(TConstArrayView<uint8> Data, int32 ExpectedDimension);

private:
	FFlatVectorIndex(int32 InDimension, faiss::IndexIDMap2* InIndex);

	int32 Dimension;
	faiss::IndexIDMap2* IndexWithIDs = nullptr;
};

} // namespace UE::SemanticSearch
