// Copyright Epic Games, Inc. All Rights Reserved.

#include "PQVectorIndex.h"

#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Set.h"
#include "Hash/Blake3.h"
#include "Implementations/SemanticSearchImplementationUtils.h"
#include "Templates/SharedPointer.h"

THIRD_PARTY_INCLUDES_START
#include "faiss/IndexPQ.h"
#include "faiss/impl/ProductQuantizer.h"
#include "faiss/index_io.h"
#include "faiss/impl/io.h"
THIRD_PARTY_INCLUDES_END

namespace UE::SemanticSearch
{

FPQVectorIndex::FPQVectorIndex(int32 InDimension, int32 InSubvectorSize, int32 InNBits)
	: Dimension(InDimension)
	, SubvectorSize(InSubvectorSize)
	, NumSubquantizers(InDimension / InSubvectorSize)
	, NBits(InNBits)
{
	check(Dimension % SubvectorSize == 0);
	faiss::IndexPQ* PQIndex = new faiss::IndexPQ(Dimension, NumSubquantizers, NBits);
	IndexWithIDs = new faiss::IndexIDMap2(PQIndex);
}

FPQVectorIndex::FPQVectorIndex(int32 InDimension, int32 InSubvectorSize, int32 InNBits, faiss::IndexIDMap2* InIndex)
	: Dimension(InDimension)
	, SubvectorSize(InSubvectorSize)
	, NumSubquantizers(InDimension / InSubvectorSize)
	, NBits(InNBits)
	, IndexWithIDs(InIndex)
{
}

FPQVectorIndex::~FPQVectorIndex()
{
	delete IndexWithIDs;
}

void FPQVectorIndex::Train(TConstArrayView<float> Vectors, int64 NumVectors)
{
	check(Vectors.Num() == NumVectors * Dimension);

	IndexWithIDs->train(NumVectors, Vectors.GetData());
	// Invalidate cached codebook hash since training changed the codebook
	bCodebookHashCached = false;
}

bool FPQVectorIndex::IsTrained() const
{
	return IndexWithIDs->is_trained;
}

void FPQVectorIndex::Add(TConstArrayView<int64> IDs, TConstArrayView<float> Vectors)
{
	check(IsTrained());
	const int64 Count = IDs.Num();
	check(Vectors.Num() == Count * Dimension);
	static_assert(sizeof(faiss::idx_t) == sizeof(int64), "FAISS idx_t size mismatch");

	IndexWithIDs->add_with_ids(
		Count,
		Vectors.GetData(),
		reinterpret_cast<const faiss::idx_t*>(IDs.GetData()));
}

void FPQVectorIndex::Remove(TConstArrayView<int64> IDs)
{
	if (IDs.Num() == 0)
	{
		return;
	}
	static_assert(sizeof(faiss::idx_t) == sizeof(int64), "FAISS idx_t size mismatch");
	faiss::IDSelectorBatch Selector(IDs.Num(), reinterpret_cast<const faiss::idx_t*>(IDs.GetData()));

	IndexWithIDs->remove_ids(Selector);
}

void FPQVectorIndex::AddQuantized(TConstArrayView<int64> IDs, TConstArrayView<uint8> Codes)
{
	check(IsTrained());
	const int64 Count = IDs.Num();
	faiss::IndexPQ* PQIndex = static_cast<faiss::IndexPQ*>(IndexWithIDs->index);
	const int64 CodeSize = PQIndex->pq.code_size;
	check(Codes.Num() == Count * CodeSize);



	const int64 BasePosition = PQIndex->ntotal;

	// Insert codes directly into the PQ index's internal storage
	PQIndex->codes.insert(PQIndex->codes.end(), Codes.GetData(), Codes.GetData() + Codes.Num());
	PQIndex->ntotal += Count;

	// Insert IDs into the IDMap and rev_map
	static_assert(sizeof(faiss::idx_t) == sizeof(int64), "FAISS idx_t size mismatch");
	const faiss::idx_t* IDsPtr = reinterpret_cast<const faiss::idx_t*>(IDs.GetData());
	IndexWithIDs->id_map.insert(IndexWithIDs->id_map.end(), IDsPtr, IDsPtr + Count);
	IndexWithIDs->ntotal += Count;

	// Update rev_map for O(1) lookups
	for (int64 i = 0; i < Count; ++i)
	{
		IndexWithIDs->rev_map[IDsPtr[i]] = BasePosition + i;
	}
}

bool FPQVectorIndex::Contains(int64 ID) const
{

	return IndexWithIDs->rev_map.count(static_cast<faiss::idx_t>(ID)) > 0;
}

bool FPQVectorIndex::TryGetEmbedding(int64 ID, TArray<float>& OutEmbedding) const
{

	if (IndexWithIDs->rev_map.count(static_cast<faiss::idx_t>(ID)) == 0)
	{
		return false;
	}

	OutEmbedding.SetNumUninitialized(Dimension);
	// Returns approximate embedding decoded from PQ codes (not exact original)
	IndexWithIDs->reconstruct(static_cast<faiss::idx_t>(ID), OutEmbedding.GetData());
	return true;
}

bool FPQVectorIndex::Update(int64 OldID, int64 NewID)
{


	auto It = IndexWithIDs->rev_map.find(static_cast<faiss::idx_t>(OldID));
	if (It == IndexWithIDs->rev_map.end())
	{
		return false;
	}

	const int64 Position = It->second;

	// Update id_map entry
	IndexWithIDs->id_map[Position] = static_cast<faiss::idx_t>(NewID);

	// Rebuild rev_map to reflect the change
	IndexWithIDs->rev_map.erase(It);
	IndexWithIDs->rev_map[static_cast<faiss::idx_t>(NewID)] = Position;

	return true;
}

int64 FPQVectorIndex::EstimateMemoryBytes() const
{

	const faiss::IndexPQ* PQIndex = static_cast<const faiss::IndexPQ*>(IndexWithIDs->index);
	const int64 Count = IndexWithIDs->ntotal;
	const int64 CodeSize = PQIndex->pq.code_size;

	// PQ codes storage
	const int64 CodesBytes = Count * CodeSize;

	// Codebook: M * ksub * dsub * sizeof(float)
	const int64 M = PQIndex->pq.M;
	const int64 Ksub = 1LL << PQIndex->pq.nbits;
	const int64 Dsub = PQIndex->pq.dsub;
	const int64 CodebookBytes = M * Ksub * Dsub * sizeof(float);

	// ID map + rev_map overhead (~24 bytes per entry: 8 for id_map + ~16 for rev_map hash entry)
	const int64 IDOverhead = Count * 24;

	return CodesBytes + CodebookBytes + IDOverhead;
}

void FPQVectorIndex::Search(
	TConstArrayView<float> QueryVector,
	int32 K,
	const TSharedRef<const TArray<int64>>& IDFilter,
	float DistanceCutoff,
	TArray<uint32>& Scratch,
	FGraphEventRef IndexReadCompleteEvent,
	TFunction<void(TArray<FSearchResult>&&)> Continuation) const
{
	check(QueryVector.Num() == Dimension);

	const bool bHasFilter = IDFilter->Num() > 0;
	const int32 NTotal = static_cast<int32>(IndexWithIDs->ntotal);
	int32 EffectiveK = K;
	if (bHasFilter)
	{
		EffectiveK = FMath::Min(FMath::Min(K, IDFilter->Num()), NTotal);
	}
	else
	{
		EffectiveK = FMath::Min(K, NTotal);
	}

	if (EffectiveK <= 0)
	{
		if (IndexReadCompleteEvent.IsValid())
		{
			IndexReadCompleteEvent->DispatchSubsequents();
		}
		Continuation(TArray<FSearchResult>{});
		return;
	}

	struct FSearchTaskState
	{
		TArray<float> QueryVector;
		TSharedPtr<const TArray<int64>> IDFilter;
		TFunction<void(TArray<FSearchResult>&&)> Continuation;
		faiss::IndexIDMap2* IndexWithIDs = nullptr;
		TArray<uint32>* Scratch = nullptr;
		// Optional event the body fires after the parallel scan finishes
		FGraphEventRef IndexReadCompleteEvent;
		int32 K = 0;
		int32 EffectiveK = 0;
		float DistanceCutoff = TNumericLimits<float>::Max();
		bool bHasFilter = false;
	};
	using FSharedState = TSharedPtr<FSearchTaskState, ESPMode::ThreadSafe>;

	FSharedState State = MakeShared<FSearchTaskState, ESPMode::ThreadSafe>();
	State->QueryVector.Append(QueryVector.GetData(), QueryVector.Num());
	State->IDFilter = IDFilter.ToSharedPtr();
	State->Continuation = MoveTemp(Continuation);
	State->IndexWithIDs = IndexWithIDs;
	State->Scratch = &Scratch;
	State->IndexReadCompleteEvent = MoveTemp(IndexReadCompleteEvent);
	State->K = K;
	State->EffectiveK = EffectiveK;
	State->DistanceCutoff = DistanceCutoff;
	State->bHasFilter = bHasFilter;

	auto SearchBody = [State]()
	{
		faiss::IndexPQ* PQIndex = static_cast<faiss::IndexPQ*>(State->IndexWithIDs->index);
		const faiss::ProductQuantizer& PQ = PQIndex->pq;
		const int32 PQ_M = static_cast<int32>(PQ.M);
		const int32 PQ_ksub = static_cast<int32>(PQ.ksub);
		const int32 CodeSize = static_cast<int32>(PQ.code_size);

		// Per-query LUT: DisTable[m * ksub + c] = distance(query subvec m, centroid c).
		TArray<float> DisTable;
		DisTable.SetNumUninitialized(PQ_M * PQ_ksub);
		PQ.compute_distance_table(State->QueryVector.GetData(), DisTable.GetData());

		const uint8_t* Codes = PQIndex->codes.data();
		const auto& RevMap = State->IndexWithIDs->rev_map;
		const faiss::idx_t* IDMap = State->IndexWithIDs->id_map.data();
		const int32 NTotal = static_cast<int32>(State->IndexWithIDs->ntotal);

		// Iteration source:
		//   filtered  → walk IDFilter, resolve user-ID to internal Pos via rev_map.
		//   unfiltered → walk every Pos in [0, ntotal), translate Pos to user-ID via id_map.
		const int32 ScanCount = State->bHasFilter ? State->IDFilter->Num() : NTotal;

	
		struct FScanTaskContext
		{
			TChunkedArray<FSearchResult> Hits;
		};
		TArray<FScanTaskContext> TaskContexts;
		constexpr int32 ScanMinBatchSize = 8192;
		ParallelForWithTaskContext(
			TEXT("FPQVectorIndex::Search Scan"),
			TaskContexts,
			ScanCount,
			ScanMinBatchSize,
			[State, &DisTable, Codes, &RevMap, IDMap, PQ_M, PQ_ksub, CodeSize]
			(FScanTaskContext& Ctx, int32 Idx)
			{
				int64 UserID;
				int64 Pos;
				if (State->bHasFilter)
				{
					UserID = (*State->IDFilter)[Idx];
					const auto It = RevMap.find(static_cast<faiss::idx_t>(UserID));
					if (It == RevMap.end())
					{
						return;
					}
					Pos = It->second;
				}
				else
				{
					Pos = Idx;
					UserID = static_cast<int64>(IDMap[Pos]);
				}

				const uint8_t* Code = Codes + Pos * CodeSize;
				float Dist = 0.0f;
				for (int32 M = 0; M < PQ_M; ++M)
				{
					Dist += DisTable[M * PQ_ksub + Code[M]];
				}

				if (Dist >= State->DistanceCutoff)
				{
					return;
				}

				FSearchResult R;
				R.ID = UserID;
				R.Distance = Dist;
				Ctx.Hits.AddElement(R);
			},
			Private::GetSearchParallelForFlags()
		);

		if (State->IndexReadCompleteEvent.IsValid())
		{
			State->IndexReadCompleteEvent->DispatchSubsequents();
		}

		// Concatenate per-task hits.
		int32 TotalHits = 0;
		for (const FScanTaskContext& Ctx : TaskContexts)
		{
			TotalHits += Ctx.Hits.Num();
		}
		TArray<FSearchResult> Results;
		Results.Reserve(TotalHits);
		for (FScanTaskContext& Ctx : TaskContexts)
		{
			Ctx.Hits.MoveToLinearArray(Results);
		}

		// Sort ascending by distance and truncate to K.
		Private::RadixSort(Results, *State->Scratch, [](const FSearchResult& R) { return R.Distance; });
		if (Results.Num() > State->K)
		{
			Results.SetNum(State->K);
		}

		State->Continuation(MoveTemp(Results));
	};

	Private::RunOrDispatchIndexWorker(MoveTemp(SearchBody));
}

TArray<uint8> FPQVectorIndex::Quantize(TConstArrayView<float> Vectors, int64 NumVectors) const
{
	check(IsTrained());
	check(Vectors.Num() == NumVectors * Dimension);


	faiss::IndexPQ* PQIndex = static_cast<faiss::IndexPQ*>(IndexWithIDs->index);
	const int64 CodeSize = PQIndex->pq.code_size;

	TArray<uint8> Codes;
	Codes.SetNumUninitialized(NumVectors * CodeSize);
	PQIndex->pq.compute_codes(Vectors.GetData(), Codes.GetData(), NumVectors);
	return Codes;
}

int64 FPQVectorIndex::GetCount() const
{

	return IndexWithIDs->ntotal;
}

TArray<uint8> FPQVectorIndex::Serialize() const
{

	faiss::VectorIOWriter Writer;
	faiss::write_index(IndexWithIDs, &Writer);

	TArray<uint8> Result;
	Result.Append(
		reinterpret_cast<const uint8*>(Writer.data.data()),
		static_cast<int32>(Writer.data.size()));
	return Result;
}

TArray<uint8> FPQVectorIndex::SerializeCodebook() const
{
	check(IsTrained());

	faiss::IndexPQ* PQIndex = static_cast<faiss::IndexPQ*>(IndexWithIDs->index);

	faiss::VectorIOWriter Writer;
	faiss::write_ProductQuantizer(&PQIndex->pq, &Writer);

	TArray<uint8> Result;
	Result.Append(
		reinterpret_cast<const uint8*>(Writer.data.data()),
		static_cast<int32>(Writer.data.size()));
	return Result;
}

FIoHash FPQVectorIndex::GetCodebookHash() const
{
	{
	
		if (bCodebookHashCached)
		{
			return CachedCodebookHash;
		}
	}

	TArray<uint8> CodebookData = SerializeCodebook();

	FBlake3 Hasher;
	Hasher.Update(CodebookData.GetData(), CodebookData.Num());
	FBlake3Hash Hash = Hasher.Finalize();


	CachedCodebookHash = FIoHash(Hash);
	bCodebookHashCached = true;
	return CachedCodebookHash;
}

TUniquePtr<FPQVectorIndex> FPQVectorIndex::DeserializeCodebook(TConstArrayView<uint8> Data, int32 ExpectedDimension)
{
	faiss::VectorIOReader Reader;
	Reader.data.assign(Data.GetData(), Data.GetData() + Data.Num());

	faiss::ProductQuantizer* PQ = faiss::read_ProductQuantizer(&Reader);
	if (!PQ || static_cast<int32>(PQ->d) != ExpectedDimension)
	{
		delete PQ;
		return nullptr;
	}

	const int32 LoadedM = static_cast<int32>(PQ->M);
	const int32 LoadedNBits = static_cast<int32>(PQ->nbits);
	const int32 LoadedSubvectorSize = ExpectedDimension / LoadedM;

	faiss::IndexPQ* PQIndex = new faiss::IndexPQ(ExpectedDimension, LoadedM, LoadedNBits);
	PQIndex->pq = std::move(*PQ);
	PQIndex->is_trained = true;
	delete PQ;

	faiss::IndexIDMap2* IDMapIndex = new faiss::IndexIDMap2(PQIndex);
	return TUniquePtr<FPQVectorIndex>(new FPQVectorIndex(ExpectedDimension, LoadedSubvectorSize, LoadedNBits, IDMapIndex));
}

TUniquePtr<FPQVectorIndex> FPQVectorIndex::Deserialize(TConstArrayView<uint8> Data, int32 ExpectedDimension)
{
	faiss::VectorIOReader Reader;
	Reader.data.assign(Data.GetData(), Data.GetData() + Data.Num());

	faiss::Index* LoadedIndex = faiss::read_index(&Reader);
	if (!LoadedIndex || LoadedIndex->d != ExpectedDimension)
	{
		delete LoadedIndex;
		return nullptr;
	}

	faiss::IndexIDMap2* IDMapIndex = static_cast<faiss::IndexIDMap2*>(LoadedIndex);
	IDMapIndex->construct_rev_map();

	faiss::IndexPQ* PQIndex = static_cast<faiss::IndexPQ*>(IDMapIndex->index);
	const int32 LoadedM = static_cast<int32>(PQIndex->pq.M);
	const int32 LoadedNBits = static_cast<int32>(PQIndex->pq.nbits);
	const int32 LoadedSubvectorSize = ExpectedDimension / LoadedM;

	return TUniquePtr<FPQVectorIndex>(new FPQVectorIndex(ExpectedDimension, LoadedSubvectorSize, LoadedNBits, IDMapIndex));
}

} // namespace UE::SemanticSearch
