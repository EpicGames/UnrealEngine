// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IQuantizedVectorIndex.h"
#include "Templates/UniquePtr.h"

THIRD_PARTY_INCLUDES_START
#include "faiss/IndexPQ.h"
#include "faiss/IndexIDMap.h"
THIRD_PARTY_INCLUDES_END

namespace UE::SemanticSearch
{

/**
 * Product Quantization vector index.
 *
 * Compresses each vector into M subvectors of SubvectorSize dimensions each,
 * quantized to nbits per subvector. With nbits=8 and SubvectorSize=8, a 768-dim
 * vector compresses from 3072 bytes to 96 bytes (~32x).
 *
 * Must be trained before use.
 *
 * @param InSubvectorSize  Number of dimensions per subquantizer. Dimension must be divisible by this.
 * @param InNBits          Bits per subquantizer code (8 = 256 centroids per subvector).
 */
class FPQVectorIndex : public IQuantizedVectorIndex
{
public:
	FPQVectorIndex(int32 InDimension, int32 InSubvectorSize = 8, int32 InNBits = 8);
	virtual ~FPQVectorIndex() override;

	FPQVectorIndex(const FPQVectorIndex&) = delete;
	FPQVectorIndex& operator=(const FPQVectorIndex&) = delete;

	// IVectorIndex batch overrides
	virtual void Train(TConstArrayView<float> Vectors, int64 NumVectors) override;
	virtual bool IsTrained() const override;
	virtual void Add(TConstArrayView<int64> IDs, TConstArrayView<float> Vectors) override;
	virtual void Remove(TConstArrayView<int64> IDs) override;
	virtual void Search(
		TConstArrayView<float> QueryVector,
		int32 K,
		const TSharedRef<const TArray<int64>>& IDFilter,
		float DistanceCutoff,
		TArray<uint32>& Scratch,
		FGraphEventRef IndexReadCompleteEvent,
		TFunction<void(TArray<FSearchResult>&&)> Continuation) const override;
	virtual int64 GetCount() const override;
	virtual int32 GetDimension() const override { return Dimension; }
	virtual TArray<uint8> Serialize() const override;
	virtual TArray<uint8> Quantize(TConstArrayView<float> Vectors, int64 NumVectors) const override;
	virtual void AddQuantized(TConstArrayView<int64> IDs, TConstArrayView<uint8> Codes) override;
	virtual bool Contains(int64 ID) const override;
	virtual bool TryGetEmbedding(int64 ID, TArray<float>& OutEmbedding) const override;
	virtual bool Update(int64 OldID, int64 NewID) override;
	virtual int64 EstimateMemoryBytes() const override;
	/** Number of subquantizers (Dimension / SubvectorSize). */
	int32 GetNumSubquantizers() const { return NumSubquantizers; }

	/** Dimensions per subquantizer. */
	int32 GetSubvectorSize() const { return SubvectorSize; }

	/** Bits per subquantizer code. */
	int32 GetNBits() const { return NBits; }

	// IQuantizedVectorIndex overrides
	virtual TArray<uint8> SerializeCodebook() const override;
	virtual FIoHash GetCodebookHash() const override;

	/** Create a trained empty index from a serialized codebook. */
	static TUniquePtr<FPQVectorIndex> DeserializeCodebook(TConstArrayView<uint8> Data, int32 ExpectedDimension);

	/** Deserialize a full index. */
	static TUniquePtr<FPQVectorIndex> Deserialize(TConstArrayView<uint8> Data, int32 ExpectedDimension);

private:
	FPQVectorIndex(int32 InDimension, int32 InSubvectorSize, int32 InNBits, faiss::IndexIDMap2* InIndex);

	int32 Dimension;
	int32 SubvectorSize;
	int32 NumSubquantizers;  // = Dimension / SubvectorSize
	int32 NBits;
	faiss::IndexIDMap2* IndexWithIDs = nullptr;
	mutable FIoHash CachedCodebookHash;
	mutable bool bCodebookHashCached = false;
};

} // namespace UE::SemanticSearch
