// Copyright Epic Games, Inc. All Rights Reserved.

#include "VectorIndexFactory.h"

#include "FlatVectorIndex.h"
#include "PQVectorIndex.h"
#include "SemanticSearchModule.h"

namespace UE::SemanticSearch
{

TSharedPtr<IVectorIndex> CreateVectorIndex(ESemanticSearchIndexType Type, int32 Dimension, const USemanticSearchSettings* Settings)
{
	if (Dimension <= 0)
	{
		return nullptr;
	}
	switch (Type)
	{
	case ESemanticSearchIndexType::PQ:
	{
		const int32 SubvectorSize = Settings ? Settings->PQSubvectorSize : 8;
		const int32 NBits = Settings ? Settings->PQNBits : 8;
		return MakeShared<FPQVectorIndex>(Dimension, SubvectorSize, NBits);
	}

	case ESemanticSearchIndexType::Flat:
	default:
		return MakeShared<FFlatVectorIndex>(Dimension);
	}
}

TSharedPtr<IVectorIndex> DeserializeVectorIndex(ESemanticSearchIndexType Type, TConstArrayView<uint8> Data, int32 Dimension)
{
	switch (Type)
	{
	case ESemanticSearchIndexType::PQ:
		return TSharedPtr<IVectorIndex>(FPQVectorIndex::Deserialize(Data, Dimension).Release());

	case ESemanticSearchIndexType::Flat:
		return TSharedPtr<IVectorIndex>(FFlatVectorIndex::Deserialize(Data, Dimension).Release());

	default:
		UE_LOGF(LogSemanticSearch, Warning, "DeserializeVectorIndex: unsupported index type %d", static_cast<uint8>(Type));
		return nullptr;
	}
}

TSharedPtr<IVectorIndex> DeserializeCodebook(ESemanticSearchIndexType Type, TConstArrayView<uint8> Data, int32 Dimension)
{
	switch (Type)
	{
	case ESemanticSearchIndexType::PQ:
		return TSharedPtr<IVectorIndex>(FPQVectorIndex::DeserializeCodebook(Data, Dimension).Release());

	case ESemanticSearchIndexType::Flat:
		return nullptr;

	default:
		UE_LOGF(LogSemanticSearch, Warning, "DeserializeCodebook: unsupported index type %d", static_cast<uint8>(Type));
		return nullptr;
	}
}

FString GetCodebookFilename(ESemanticSearchIndexType Type)
{
	switch (Type)
	{
	case ESemanticSearchIndexType::PQ:
		return TEXT("PQCodebook.bin");

	case ESemanticSearchIndexType::Flat:
	default:
		return FString();
	}
}

} // namespace UE::SemanticSearch
