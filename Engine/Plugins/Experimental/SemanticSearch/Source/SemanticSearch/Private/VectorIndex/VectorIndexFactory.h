// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IVectorIndex.h"
#include "Settings/SemanticSearchSettings.h"
#include "Templates/SharedPointer.h"

namespace UE::SemanticSearch
{

/** Create a fresh vector index for the given type. Quantized types start untrained. */
TSharedPtr<IVectorIndex> CreateVectorIndex(ESemanticSearchIndexType Type, int32 Dimension, const USemanticSearchSettings* Settings);

/** Deserialize a full index (with vectors) from bytes. Returns nullptr on failure. */
TSharedPtr<IVectorIndex> DeserializeVectorIndex(ESemanticSearchIndexType Type, TConstArrayView<uint8> Data, int32 Dimension);

/** Deserialize a trained-but-empty codebook. Returns nullptr for Flat or on failure. */
TSharedPtr<IVectorIndex> DeserializeCodebook(ESemanticSearchIndexType Type, TConstArrayView<uint8> Data, int32 Dimension);

/** Get the codebook filename for an index type (e.g. "PQCodebook.bin"). Returns empty for types without codebooks. */
FString GetCodebookFilename(ESemanticSearchIndexType Type);

} // namespace UE::SemanticSearch
