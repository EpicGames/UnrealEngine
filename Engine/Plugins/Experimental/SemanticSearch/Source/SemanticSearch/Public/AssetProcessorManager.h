// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StripedMap.h"
#include "Containers/UnrealString.h"
#include "Delegates/IDelegateInstance.h"
#include "DerivedDataCacheKey.h"
#include "HAL/CriticalSection.h"
#include "Interfaces/IAssetProcessor.h"
#include "IO/IoHash.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/TopLevelAssetPath.h"

namespace UE::SemanticSearch
{

namespace Private
{
class FSemanticSearchBuildCacheTask;
}

struct FAssetCaptionResult
{
	FString Caption;
	TArray<FString> Keywords;
};

struct FAssetEmbeddingResult
{
	TArray<float> Embedding;
};

struct FAssetQuantizedEmbeddingResult
{
	TArray<uint8> QuantizedCodes;
};

/** Combined result for a single-call indexing data fetch (caption + embedding or quantized codes). */
struct FAssetIndexingResult
{
	FString Caption;
	TArray<FString> Keywords;
	TArray<float> Embedding;        // Non-empty for flat path
	TArray<uint8> QuantizedCodes;   // Non-empty for quantized path
};

class FAssetProcessorManager
{
public:

	SEMANTICSEARCH_API static FAssetProcessorManager& Get();
	
	SEMANTICSEARCH_API bool GetCaptionData(const FAssetData& InAssetData, TUniqueFunction<void(FAssetCaptionResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable);
	SEMANTICSEARCH_API bool GetEmbeddingData(const FAssetData& InAssetData, TUniqueFunction<void(FAssetEmbeddingResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable, bool bBuildOnMiss = true);
	SEMANTICSEARCH_API bool GetQuantizedEmbeddingData(const FAssetData& InAssetData, const FIoHash& CodebookHash,
		TUniqueFunction<void(FAssetQuantizedEmbeddingResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable, bool bBuildOnMiss = true);

	/**
	 * Fetch caption + float embedding in a single call (both DDC requests issued in parallel).
	 * The callback fires once, after both have completed. Reason classifies any failure.
	 */
	SEMANTICSEARCH_API bool GetIndexingData(const FAssetData& InAssetData,
		TUniqueFunction<void(FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable, bool bBuildOnMiss = true);

	/**
	 * Fetch caption + quantized codes in a single call (both DDC requests issued in parallel).
	 * The callback fires once, after both have completed. Reason classifies any failure.
	 */
	SEMANTICSEARCH_API bool GetQuantizedIndexingData(const FAssetData& InAssetData, const FIoHash& CodebookHash,
		TUniqueFunction<void(FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)>&& InOnDataAvailable, bool bBuildOnMiss = true);

	/**
	 * Batch-fetch indexing data for multiple assets in a single DDC Get() call.
	 * On cache hit, the per-asset callback fires with the result.
	 * On cache miss, falls back to per-asset BuildDerivedData().
	 */
	SEMANTICSEARCH_API void BatchGetIndexingData(
		TConstArrayView<FAssetData> Assets,
		TFunction<void(const FAssetData& Asset, FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)> OnAssetResult,
		bool bBuildOnMiss = true);

	/** Same as BatchGetIndexingData but for the quantized path. */
	SEMANTICSEARCH_API void BatchGetQuantizedIndexingData(
		TConstArrayView<FAssetData> Assets,
		const FIoHash& CodebookHash,
		TFunction<void(const FAssetData& Asset, FAssetIndexingResult&&, FString&&, EAssetIndexFailureReason)> OnAssetResult,
		bool bBuildOnMiss = true);

	/** Cancel all in-flight build tasks and clear the jobs map. */
	SEMANTICSEARCH_API void CancelAllTasks();

	/** Returns the processor for the asset's class. May return null. */
	SEMANTICSEARCH_API TSharedPtr<IAssetProcessor> GetProcessorForAsset(const FAssetData& InAssetData) const;

	/** Returns short class names of every asset class currently handled by a registered processor,
	 *  including derived subclasses populated via PopulateChildClassesCache.
	 */
	SEMANTICSEARCH_API TSet<FName> GetSupportedClassNames() const;

	/** Returns the total number of supported assets in indexed folders. Cached; invalidated on asset registry changes. */
	SEMANTICSEARCH_API int32 GetSupportedAssetCount() const;

	/** Force-invalidate the supported asset count cache (e.g. when IndexedPaths changes). */
	SEMANTICSEARCH_API void InvalidateSupportedAssetCount();

	/** Clear all cached build tasks. Necessary before re-indexing to ensure fresh DDC lookups. */
	SEMANTICSEARCH_API void ClearBuildCache();

	template<typename FAssetProcessor>
	void RegisterAssetProcessor();

	template<typename FAssetProcessor>
	void UnregisterAssetProcessor();

private:

	FAssetProcessorManager();
	~FAssetProcessorManager();

	SEMANTICSEARCH_API void RegisterAssetProcessor(TSharedRef<IAssetProcessor>&& AssetProcessor);
	SEMANTICSEARCH_API void UnregisterAssetProcessor(UClass& Class);
	void PopulateChildClassesCache(UClass& Class, TSharedPtr<IAssetProcessor> Processor);
	int32 ComputeSupportedAssetCount() const;
	void InvalidateSupportedAssetCount(const FAssetData&);

	bool GetOrCreateTask(const FAssetData& InAssetData, TFunctionRef<void(const TSharedRef<Private::FSemanticSearchBuildCacheTask>&)> InApply);

	void OnObjectsReinstanced(const TMap<UObject*, UObject*>& InReplacementObjects);
	void OnModuleUnloaded(TConstArrayView<UPackage*> UnloadingPackages);
	void OnHotReloadComplete();

	FDelegateHandle OnObjectsReinstancedHandle;
	FDelegateHandle OnModuleUnloadedHandle;
	FDelegateHandle OnHotReloadCompleteHandle;
	FDelegateHandle OnAssetAddedHandle;
	FDelegateHandle OnAssetRemovedHandle;

	mutable TAtomic<int32> CachedSupportedAssetCount{-1};
	mutable FRWLock AssetTypeToProcessorLock;
	TMap<FTopLevelAssetPath, TSharedPtr<IAssetProcessor>> AssetTypeToProcessor;

	TStripedMap<32, DerivedData::FCacheKey, TSharedRef<Private::FSemanticSearchBuildCacheTask>> BuildsJobs;
};

template<typename FAssetProcessor>
void FAssetProcessorManager::RegisterAssetProcessor()
{
	static_assert(TIsDerivedFrom<FAssetProcessor, IAssetProcessor>::Value, "Asset Processors must derive from IAssetProcessor");
	RegisterAssetProcessor(MakeShared<FAssetProcessor>());
}

template<typename FAssetProcessor>
void FAssetProcessorManager::UnregisterAssetProcessor()
{
	static_assert(TIsDerivedFrom<FAssetProcessor, IAssetProcessor>::Value, "Asset Processors must derive from IAssetProcessor");
	FAssetProcessor AssetProcessor;
	UnregisterAssetProcessor(AssetProcessor.GetSupportedClass());
}

}