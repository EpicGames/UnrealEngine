// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Interfaces/IVectorIndex.h"
#include "IO/IoHash.h"
#include "Settings/SemanticSearchSettings.h"
#include "Tasks/Task.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Delegates/Delegate.h"

#include <atomic>

namespace UE::SemanticSearch
{

struct FHybridSearchResult
{
	int64 ID = -1;
	float RRFScore = 0.0f;
	float VectorDistance = -1.0f;	// -1 if not in vector results
	float BM25Score = -1.0f;		// -1 if not in BM25 results
};

/**
 * Classification of an indexing failure. Populated by the layer that knows the cause
 * (processor vs. embedding provider) and carried through to FHybridSearchIndex::MarkFailed
 * so retryable (Provider) failures can be separated from permanent (PreProcessor) ones.
 */
enum class EAssetIndexFailureReason : uint8
{
	None         = 0, // Success.
	PreProcessor = 1, // Error before the backend call (e.g. missing thumbnail). Permanent, persisted.
	Provider     = 2, // Error from the backend (connection, HTTP status, parse, server error). Retryable.
};

/** Pre-processor failures are permanent; Provider failures are retryable. */
inline bool IsPreProcessorFailure(EAssetIndexFailureReason Reason)
{
	return Reason == EAssetIndexFailureReason::PreProcessor;
}

struct FSemanticSearchIndexStats
{
	ESemanticSearchIndexType IndexType = ESemanticSearchIndexType::Flat;
	int32 VectorCount = 0;
	int32 BM25Count = 0;
	int32 Dimension = 0;
	int64 EstimatedVectorMemoryBytes = 0;
	int64 EstimatedBM25MemoryBytes = 0;
	bool bIsTrained = false;
	bool bSupportsQuantization = false;
	bool bIsInitialized = false;
	int32 SupportedAssetCount = 0;
	/** Retryable failures (e.g. provider/connection errors). Cleared on restart. */
	int32 FailedCount = 0;
	/** Permanent failures (e.g. asset has no thumbnail). Persisted via sidecar file; skipped by auto-retry. */
	int32 PreProcessorFailedCount = 0;
	bool bIsIndexing = false;
};

class FBM25Index;
struct FCommandQueueData;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnIndexChanged, bool /*bSuccess*/);

/**
 * Singleton that owns all search indices (vector + BM25) and fuses results
 * using Reciprocal Rank Fusion (RRF).
 *
 * All mutations, searches, and index queries are serialized through an internal
 * command queue with a dedicated consumer thread. Public methods enqueue work
 * (microsecond lock) and return immediately; async operations deliver results
 * via game-thread callbacks. GetCachedIndexStats() provides a lock-free snapshot
 * of index metrics that is safe to call from any thread.
 */
class FHybridSearchIndex
{
public:
	SEMANTICSEARCH_API static FHybridSearchIndex& Get();

	// --- Command queue management ---

	/** Start the background consumer thread. */
	SEMANTICSEARCH_API void StartCommandQueue();

	/** Stop the background consumer thread and join. */
	SEMANTICSEARCH_API void StopCommandQueue();

	/** Discard all pending mutations (used during cancel). */
	SEMANTICSEARCH_API void DiscardPendingMutations();

	// --- Mutations (enqueued to background thread, return immediately) ---

	/** Add a single asset to both indices. Lazily creates vector index on first call. */
	SEMANTICSEARCH_API void Add(FAssetData Asset, TArray<float> Embedding,
		FString Caption, TArray<FString> Keywords, uint32 IndexId);

	/** Add a single asset using pre-quantized codes (no float embedding). */
	SEMANTICSEARCH_API void AddQuantized(FAssetData Asset, TArray<uint8> QuantizedCodes,
		FString Caption, TArray<FString> Keywords, uint32 IndexId);

	/**
	 * Record that an asset failed to index. The reason decides which bucket it lands in:
	 * - EAssetIndexFailureReason::PreProcessor  → permanent bucket (persisted; not auto-retried).
	 * - EAssetIndexFailureReason::Provider      → retryable bucket (in-session; cleared by "Retry failed").
	 */
	SEMANTICSEARCH_API void MarkFailed(int64 AssetId, EAssetIndexFailureReason Reason, uint32 IndexId);

	/** Remove an asset ID from both failed sets. Used by the retry button. */
	SEMANTICSEARCH_API void ClearFailedState(int64 AssetId);

	/**
	 * Purge pre-processor-failed IDs that are not in the given set of currently-valid IDs.
	 * Called after OnFilesLoaded to drop stale entries left over from deletions or re-saves
	 * (where the DDC key changed and the old ID is no longer meaningful).
	 */
	SEMANTICSEARCH_API void PurgePreProcessorFailedNotIn(TSet<int64> ValidIDs);

	/** Remove a single asset from both indices. */
	SEMANTICSEARCH_API void Remove(FAssetData Asset);

	/** Remove by path string (for when the asset has already been deleted). */
	SEMANTICSEARCH_API void RemoveByPath(FString AssetPath);

	/** Remove by pre-computed ID. */
	SEMANTICSEARCH_API void RemoveById(int64 AssetId);

	/** Replace the vector index. Used during index type switching. */
	SEMANTICSEARCH_API void SetVectorIndex(TSharedPtr<IVectorIndex> NewIndex);

	/** Clear the BM25 text index. */
	SEMANTICSEARCH_API void ClearBM25();

	/**
	 * Save the full index (vector + BM25 + package hashes) to a file, and the pre-processor-failed
	 * ID list to a sidecar file alongside it.
	 * @param FilePath					Destination file path for the main index.
	 * @param PreProcessorFailedPath	Sidecar file path for the pre-processor-failed IDs.
	 * @param PackageHashes				Per-asset PackageSavedHash for staleness detection on next load.
	 */
	SEMANTICSEARCH_API void Save(FString FilePath, FString PreProcessorFailedPath, TMap<int64, FIoHash> PackageHashes);

	// --- Search (enqueued with high priority, results via callback on game thread) ---

	/**
	 * Async search. Fuses vector + BM25 results via RRF.
	 * @param QueryText			Text query for BM25 search.
	 * @param QueryEmbedding	Embedding for vector search. Can be empty if not yet available.
	 * @param K					Number of results to return.
	 * @param IDFilter			If non-empty, only search among these IDs.
	 * @param DistanceCutoff	Forwarded to the vector search; results with
	 *							Distance >= cutoff are dropped before fusion.
	 *							Pass TNumericLimits<float>::Max() for "no cutoff".
	 * @param Callback			Receives results on the game thread.
	 */
	SEMANTICSEARCH_API void SearchAsync(FString QueryText, TArray<float> QueryEmbedding,
		int32 K, TArray<int64> IDFilter, float DistanceCutoff,
		TFunction<void(TArray<FHybridSearchResult>&&)> Callback);

	/**
	 * Async quantize. Encodes a float embedding into quantized codes on the consumer thread,
	 * avoiding races with concurrent index mutations.
	 * @param Embedding		Float embedding to quantize.
	 * @param Callback		Receives quantized codes on the consumer thread. bSuccess is false
	 *						if no trained quantized index is available.
	 */
	SEMANTICSEARCH_API void QuantizeAsync(TArray<float> Embedding,
		TFunction<void(TArray<uint8>&&, bool bSuccess)> Callback);

	/**
	 * Async contains check. Tests which IDs are present in the index on the consumer thread.
	 * @param IDs		IDs to check.
	 * @param Callback	Receives the set of IDs that are present, on the game thread.
	 */
	SEMANTICSEARCH_API void ContainsAsync(TArray<int64> IDs,
		TFunction<void(TSet<int64>&&)> Callback);

	/**
	 * Async extraction of all stored vectors for training. Runs on the consumer thread.
	 * @param CandidateIDs	IDs to try extracting embeddings for.
	 * @param Callback		Receives matched IDs, concatenated vectors, and dimension on the game thread.
	 */
	SEMANTICSEARCH_API void ExtractEmbeddingsAsync(TArray<int64> CandidateIDs,
		TFunction<void(TArray<int64>&& IDs, TArray<float>&& Vectors, int32 Dimension)> Callback);

	// --- Read-only queries ---

	/** Whether BM25 hybrid search is enabled (from settings). */
	bool IsHybridEnabled() const;

	/** Reset BM25 and failed sets (enqueued to consumer thread). Used when re-indexing all assets from scratch. */
	SEMANTICSEARCH_API void ResetBM25AndFailedState();

	/**
	 * Async failed-set check. Returns the union of retryable and pre-processor failed IDs
	 * on the game thread. Used by pre-index skip paths that treat any failure as "don't retry".
	 */
	SEMANTICSEARCH_API void IsFailedAsync(TArray<int64> IDs,
		TFunction<void(TSet<int64>&&)> Callback);

	/**
	 * Async retryable-failed check. Returns only the in-session retryable failed IDs
	 * (NOT pre-processor failures). Used by the "Retry failed" button so permanent
	 * failures are excluded from retries.
	 */
	SEMANTICSEARCH_API void IsRetryableFailedAsync(TArray<int64> IDs,
		TFunction<void(TSet<int64>&&)> Callback);
	/**
	 * Get a snapshot of index stats. Updated atomically by the consumer thread
	 * after each mutation batch — safe to call from any thread with no locking.
	 * Fills IndexType and SupportedAssetCount are left to the caller.
	 */
	SEMANTICSEARCH_API FSemanticSearchIndexStats GetCachedIndexStats() const;

	/** Get the codebook hash from cached stats. Returns default FIoHash if not quantized/trained. */
	SEMANTICSEARCH_API FIoHash GetCachedCodebookHash() const;

	/**
	 * Load a previously saved index from a file. Replaces current index state.
	 * Must be called before StartCommandQueue (not thread-safe with the consumer thread).
	 * @param OutPackageHashes	Filled with per-asset PackageSavedHash from save time (for staleness checks).
	 */
	SEMANTICSEARCH_API bool LoadFromFile(const FString& FilePath, TMap<int64, FIoHash>& OutPackageHashes);

	/**
	 * Load the pre-processor-failed ID list from a sidecar file. Missing file = empty set, no warning
	 * (expected on first run and after user deletes the file to reset). Corrupt file logs one warning
	 * and initializes empty. Must be called before StartCommandQueue.
	 */
	SEMANTICSEARCH_API void LoadPreProcessorFailedFromFile(const FString& FilePath);

	/** Get the path to the shared codebook file for the given index type (in plugin Resources/). */
	SEMANTICSEARCH_API static FString GetCodebookPath(ESemanticSearchIndexType Type);

	/** Get the current index ID (changes on index swap/invalidation). */
	SEMANTICSEARCH_API uint32 GetIndexId() const;

	/** Invalidate the current IndexId so stale callbacks are dropped. */
	SEMANTICSEARCH_API void InvalidateIndexId();

	/** Ensure the index is initialized with a valid (possibly empty) vector index. */
	SEMANTICSEARCH_API void EnsureInitialized(int32 EmbeddingDimension);

	/** Delegate broadcast when the index changes (e.g. after switching type or retraining). Game thread only. */
	FOnIndexChanged& OnIndexChanged()
	{
		check(IsInGameThread());
		return OnIndexChangedDelegate;
	}

private:
	FHybridSearchIndex();
	~FHybridSearchIndex();

	// --- Task graph processing ---

	void ProcessCommandBatch();
	void EnsureProcessingTask(UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::BackgroundNormal);
	void UpdateCachedStats();

	// --- Internal accessors (consumer thread or pre-queue only) ---

	TSharedPtr<IVectorIndex> GetVectorIndex() const;
	bool Contains(const FAssetData& Asset) const;
	FBM25Index& GetBM25Index();

	// --- Direct implementations (only called by ProcessCommandBatch) ---

	void AddDirect(const FAssetData& Asset,
		TConstArrayView<float> Embedding, FStringView Caption, TConstArrayView<FString> Keywords);
	void AddDirect(TConstArrayView<FAssetData> Assets,
		TConstArrayView<float> AllEmbeddings,
		TConstArrayView<FString> Captions,
		const TArray<TArray<FString>>& AllKeywords);

	void AddQuantizedDirect(const FAssetData& Asset,
		TConstArrayView<uint8> QuantizedCodes, FStringView Caption, TConstArrayView<FString> Keywords);
	void AddQuantizedDirect(TConstArrayView<FAssetData> Assets,
		const TArray<TArray<uint8>>& AllQuantizedCodes,
		TConstArrayView<FString> Captions,
		const TArray<TArray<FString>>& AllKeywords);

	void RemoveDirect(const FAssetData& Asset);
	void RemoveDirect(FStringView AssetPath);
	void RemoveDirect(int64 AssetID);
	void RemoveDirect(TConstArrayView<int64> IDs);

	void SetVectorIndexDirect(TSharedPtr<IVectorIndex> NewIndex);

	void MarkFailedDirect(int64 AssetID, EAssetIndexFailureReason Reason);
	void ClearFailedStateDirect(int64 AssetID);
	void PurgePreProcessorFailedNotInDirect(const TSet<int64>& ValidIDs);

	void SearchDirect(
		FString QueryText,
		TArray<float> QueryEmbedding,
		int32 K,
		TArray<int64> IDFilter,
		float DistanceCutoff,
		FGraphEventArray* OutIndexReadEvents,
		TFunction<void(TArray<FHybridSearchResult>&&)> Continuation) const;

	TArray64<uint8> SerializeIndexForSave(const TMap<int64, FIoHash>& PackageHashes) const;
	TArray64<uint8> SerializePreProcessorFailedForSave() const;

	static void WriteBytesToFile(const FString& FilePath, TArray64<uint8> Bytes);

	// --- Data (both failed sets owned exclusively by consumer thread) ---

	/** Retryable failures. In-session only; cleared by the "Retry failed" button and on force-rebuild. */
	TSet<int64> FailedAssets;

	/** Permanent failures. Persisted via sidecar file; skipped by auto-retry and the "Retry failed" button. */
	TSet<int64> PreProcessorFailedAssets;
	std::atomic<bool> bInitialized{false};
	std::atomic<uint32> IndexId{0};
	TSharedPtr<IVectorIndex> VectorIndex;
	TUniquePtr<FBM25Index> BM25Index;
	TUniquePtr<FCommandQueueData> QueueData;
	FOnIndexChanged OnIndexChangedDelegate;
};

/** Compute the stable int64 ID used for an asset in the indices. */
SEMANTICSEARCH_API int64 GetAssetIndexID(const FAssetData& Asset);
SEMANTICSEARCH_API int64 GetAssetIndexID(FStringView AssetPath);

} // namespace UE::SemanticSearch
