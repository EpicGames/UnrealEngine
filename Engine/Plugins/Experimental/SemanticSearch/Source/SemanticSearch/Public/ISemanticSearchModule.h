// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "HybridSearchIndex.h"
#include "Interfaces/IEmbeddingProvider.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Settings/SemanticSearchSettings.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

namespace UE::SemanticSearch
{
	class ISemanticSearchModule : public IModuleInterface
	{
	public:
		static ISemanticSearchModule& Get()
		{
			static const FName ModuleName = "SemanticSearch";
			return FModuleManager::LoadModuleChecked<ISemanticSearchModule>(ModuleName);
		}

		virtual void RegisterEmbeddingProvider(TUniquePtr<IEmbeddingProvider>&& EmbeddingProvider, bool bRebuildIndex) = 0;

		/** Returns the active embedding provider, or nullptr if none has been registered. */
		virtual TSharedPtr<IEmbeddingProvider> GetEmbeddingProvider() const = 0;

		/**
		 * Load the saved index, start the consumer queue, and subscribe to asset-registry events
		 * using the currently registered embedding provider. Idempotent — repeat calls are no-ops.
		 *
		 * StartupModule() invokes this automatically unless the console variable
		 * `SemanticSearch.DelayInitialization=true` is set. Modules that want to install a custom
		 * embedding provider before the index is built must set this CVar via a config file —
		 * for example, add to a project's DefaultEngine.ini:
		 *
		 *     [ConsoleVariables]
		 *     SemanticSearch.DelayInitialization=1
		 *
		 * Then register the provider via RegisterEmbeddingProvider() and call Initialize() to
		 * finalize startup. Must be invoked on the game thread.
		 */
		virtual void Initialize() = 0;

		/** True once Initialize() has run successfully. */
		virtual bool IsInitialized() const = 0;
		/** Index a single asset, always building from the backend if not already cached. */
		virtual void IndexAsset(const FAssetData& AssetData) = 0;

		/** Index a batch of assets. Filters through processor check, uses batch DDC paths, and shows progress. */
		virtual void IndexAssets(TConstArrayView<FAssetData> Assets) = 0;

		/** Returns true if the asset is in a folder configured for indexing. */
		SEMANTICSEARCH_API static bool IsInIndexedFolder(const FAssetData& AssetData);

		/** Returns true if the given package path is in a folder configured for indexing. */
		SEMANTICSEARCH_API static bool IsInIndexedFolder(FStringView PackagePath);
		/** Get current index statistics. */
		virtual FSemanticSearchIndexStats GetIndexStats() const = 0;

		/** Switch to a different index type. Re-indexes all assets. */
		virtual void SwitchIndexType(ESemanticSearchIndexType NewType) = 0;

		/** Re-index all supported assets. When bForceBuild is true, clears the index and always builds on DDC miss. */
		virtual void IndexAllAssets(bool bForceBuild = false) = 0;

		/** Retrain the current quantized index from scratch (new codebook). No-op for flat index. */
		virtual void RetrainIndex() = 0;

		/** Cancel all in-flight indexing tasks, drop the job queue, and dismiss progress UI. */
		virtual void CancelIndexing() = 0;

		/** Returns true while a batch indexing operation is in flight (safe to call from any thread). */
		virtual bool IsIndexingInProgress() const = 0;

		/**
		 * Mark an indexing-related operation as pending. Call from UI button handlers immediately
		 * on click so `FSemanticSearchIndexStats::bIsIndexing` flips true synchronously (before the
		 * async enumeration / filter / DDC work that would normally set it), keeping the button
		 * visibly disabled instead of clickable while we spin up. Cleared automatically when real
		 * indexing activity starts (OnIndexingStarted / OnBatchIndexingStarted), when no work ends
		 * up being queued, or on a safety timeout.
		 */
		virtual void SetIndexingPending() = 0;

		/**
		 * Perform a hybrid search asynchronously via the index command queue.
		 * The callback is invoked on the game thread with the results.
		 *
		 * @param DistanceCutoff  Forwarded to the vector search; results with
		 *                        Distance >= cutoff are dropped before fusion.
		 *                        Pass TNumericLimits<float>::Max() for "no cutoff".
		 */
		virtual void SearchAsync(
			const FString& QueryText,
			TConstArrayView<float> QueryEmbedding,
			int32 K,
			TConstArrayView<int64> IDFilter,
			float DistanceCutoff,
			TFunction<void(TArray<FHybridSearchResult>&&)> Callback) = 0;
	};
}
