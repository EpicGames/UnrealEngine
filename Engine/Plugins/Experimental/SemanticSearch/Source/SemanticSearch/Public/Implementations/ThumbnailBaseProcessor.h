// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IAssetProcessor.h"

#include "Serialization/MemoryHasher.h"
#include "Templates/SharedPointer.h"

class FJsonObject;
struct FAssetData;

namespace UE::SemanticSearch
{

/**
 * Reusable base class for asset processors that operate via on-disk thumbnails.
 *
 * Derived classes must:
 *  - Implement GetProcessSubBucketName() tell the system where to store the derived
 *  - Implement GetMetadata(InAsset) to supply structured per-asset metadata.
 *
 * Derived classes may optionally override GetRevision() to bump cached data if they their implementation details.
 */
class SEMANTICSEARCH_API FThumbnailBaseAssetProcessor : public IAssetProcessor
{
public:
	/** Returns structured metadata for the asset. Called from GenerateCaptionRequest. */
	virtual TSharedPtr<FJsonObject> GetMetadata(const TSharedRef<const FAssetData>& InAsset) const = 0;

	/** Bump this to invalidate previously cached derived data. */
	virtual int32 GetRevision() const { return 1; }

	virtual bool GenerateAssetHash(const FAssetData& Asset, FMemoryHasherBlake3& InHasher) const override;
	virtual void GenerateCaptionRequest(const TSharedRef<const FAssetData>& InAsset, bool bCanDisruptUser, DerivedData::FRequestOwner& RequestOwner, FOnRequestComplete OnRequestComplete) const override;
};

}
