// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "HybridSearchIndex.h"
#include "Interfaces/IEmbeddingProvider.h"
#include "Serialization/MemoryHasher.h"
#include "Templates/Function.h"

class UClass;
struct FAssetData;


namespace UE::DerivedData
{
	class FRequestOwner;
}

namespace UE::SemanticSearch
{
struct FCaptionRequest;

/**
 * Processor-generated caption-request result. On success, bHasGeneratedCaptionRequest=true and Reason=None.
 * On failure, bHasGeneratedCaptionRequest=false and Reason classifies the failure:
 * - PreProcessor: thumbnail missing, unsupported media format, etc. Permanent.
 * - Provider:     reserved; processors normally only emit PreProcessor failures.
 */
using FOnRequestComplete = TUniqueFunction<void(bool bHasGeneratedCaptionRequest, FCaptionRequest&& CaptionRequest, FString&& ErrorMessage, EAssetIndexFailureReason Reason)>;

class IAssetProcessor
{
public:

	/**
	 * Used to separate the generated data into their own bucket
	 * Use using the name of your type is acceptable value if it is unique
	 */
	constexpr virtual FStringView GetProcessSubBucketName() const = 0;

	virtual UClass& GetSupportedClass() const = 0;
	virtual bool SupportDerivedClasses() const = 0;

	virtual bool GenerateAssetHash(const FAssetData& Asset, FMemoryHasherBlake3& InHasher) const = 0;
	virtual void GenerateCaptionRequest(const TSharedRef<const FAssetData>& InAsset, bool bCanDisruptUser, DerivedData::FRequestOwner& RequestOwner, FOnRequestComplete OnRequestComplete) const = 0;

	virtual ~IAssetProcessor() = default;
};

}