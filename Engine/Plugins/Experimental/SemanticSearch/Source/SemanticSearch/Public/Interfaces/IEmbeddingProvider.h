// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "HybridSearchIndex.h"
#include "Templates/Function.h"

namespace UE::DerivedData
{
	class FRequestOwner;
}

namespace UE::SemanticSearch
{
	/**
	 * A single media payload attached to a caption request.
	 */
	struct FAssetMedia
	{
		/** MIME type: "image/png", "application/json", "audio/mpeg", etc. */
		FString MimeType;

		/** Raw media bytes */
		TArray<uint8> Data;
	};

	/**
	 * Request to generate a caption (and keywords) for an asset.
	 * Can include media content and structured metadata.
	 */
	struct FCaptionRequest
	{
		FString AssetPath;
		FString AssetType;

		/** Asset content — can be combination of images, text, JSON */
		TArray<FAssetMedia> AssetMedia;

		/** Structured metadata (asset dimension, properties, etc.) */
		TSharedPtr<FJsonObject> Metadata;
	};

	/**
	 * Response from caption generation.
	 * Success if ErrorMessage is empty (FailureReason == None).
	 */
	struct FCaptionResponse
	{
		FString ErrorMessage;

		/** Classification of the failure, if any. Populated by the provider layer. */
		EAssetIndexFailureReason FailureReason = EAssetIndexFailureReason::None;

		FString Caption;

		TArray<FString> Keywords;
	};

	/**
	 * Response from embedding generation.
	 * Success if ErrorMessage is empty (FailureReason == None).
	 */
	struct FEmbeddingResponse
	{
		FString ErrorMessage;

		/** Classification of the failure, if any. Populated by the provider layer. */
		EAssetIndexFailureReason FailureReason = EAssetIndexFailureReason::None;

		TArray<float> Embedding;

		/** Version of the model that generated this embedding */
		FString ModelVersion;
	};


	using FOnCaptionComplete = TUniqueFunction<void (FCaptionResponse&&)>;

	using FOnEmbeddingComplete = TUniqueFunction<void (FEmbeddingResponse&&)>;

	/**
	 * Interface for embedding generation providers.
	 * Can support both remote (HTTP API) and local (on-device) implementations.
	 */
	class IEmbeddingProvider
	{
	public:
		virtual ~IEmbeddingProvider() = default;

		/** Embedding vector dimensionality */
		virtual int32 GetEmbeddingDimension() const = 0;

		/**
		 * Generate a caption and keywords for an asset.
		 * Called during indexing to describe the asset content.
		 *
		 * @param Request The caption request with asset media and metadata
		 * @param OnComplete Callback invoked when caption is ready
		 * @param RequestOwner The request owner is optional argument to help the derived data build track it subtasks
		 */
		virtual void GenerateCaptionAsync(
			FCaptionRequest&& Request,
			FOnCaptionComplete&& OnComplete,
			DerivedData::FRequestOwner* RequestOwner) = 0;

		/**
		 * Generate embedding for a text string.
		 * Called during indexing (with caption text) and at search time (with query text).
		 *
		 * @param Text The text to embed (caption or search query)
		 * @param OnComplete Callback invoked when embedding is ready
		 * @param RequestOwner The request owner is optional argument to help the derived data build track it subtasks
		 */
		virtual void GenerateEmbeddingAsync(
			const FStringView Text,
			FOnEmbeddingComplete&& OnComplete,
			DerivedData::FRequestOwner* RequestOwner) = 0;

		/**
		 * Get a hash that represents the configuration of this embedding provider
		 */
		virtual uint32 GetConfigHash() const = 0;

		/**
		 * Cancel all pending requests that have not yet been dispatched.
		 * Default implementation is no-op for providers that don't queue requests.
		 */
		virtual void CancelAllPendingRequests() {}
	};
}
