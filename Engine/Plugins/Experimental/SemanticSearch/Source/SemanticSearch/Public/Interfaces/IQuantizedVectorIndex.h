// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IVectorIndex.h"
#include "IO/IoHash.h"

namespace UE::SemanticSearch
{

/**
 * Interface for vector indices that support quantization (PQ, RaBitQ, etc.).
 *
 * Adds quantization, codebook serialization, and pre-quantized insertion
 * to the base IVectorIndex interface.
 *
 * Callers should check IVectorIndex::SupportsQuantization() before casting
 * to this type via static_cast.
 */
class IQuantizedVectorIndex : public IVectorIndex
{
public:
	virtual bool SupportsQuantization() const override final { return true; }

	/**
	 * Quantize vectors into compressed form (e.g. PQ codes, RaBitQ codes).
	 */
	virtual TArray<uint8> Quantize(TConstArrayView<float> Vectors, int64 NumVectors) const = 0;

	TArray<uint8> Quantize(TConstArrayView<float> Vector) const
	{
		return Quantize(Vector, 1);
	}

	/**
	 * Add pre-quantized vectors directly, skipping encoding.
	 */
	virtual void AddQuantized(TConstArrayView<int64> IDs, TConstArrayView<uint8> Codes) = 0;

	void AddQuantized(int64 ID, TConstArrayView<uint8> Codes)
	{
		AddQuantized(TConstArrayView<int64>(&ID, 1), Codes);
	}

	/** Serialize the trained codebook/quantizer state (no vector data). */
	virtual TArray<uint8> SerializeCodebook() const = 0;

	/** Blake3 hash of the serialized codebook. Used for DDC invalidation and save-file validation. */
	virtual FIoHash GetCodebookHash() const = 0;
};

} // namespace UE::SemanticSearch
