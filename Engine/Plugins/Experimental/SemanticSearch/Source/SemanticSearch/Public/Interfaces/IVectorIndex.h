// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

namespace UE::SemanticSearch
{

struct FSearchResult
{
	int64 ID = -1;
	float Distance = 0.0f;
};

/**
 * Abstract interface for vector similarity search indices.
 */
class IVectorIndex
{
public:
	virtual ~IVectorIndex() = default;

	// --- Virtual batch methods ---

	/** Train the index on representative vectors. No-op for flat indices. */
	virtual void Train(TConstArrayView<float> Vectors, int64 NumVectors) = 0;

	/** Whether the index has been trained and is ready for Add/Search. */
	virtual bool IsTrained() const = 0;

	/** Add multiple vectors with corresponding IDs. Index must be trained first. */
	virtual void Add(TConstArrayView<int64> IDs, TConstArrayView<float> Vectors) = 0;

	/** Remove vectors by IDs. */
	virtual void Remove(TConstArrayView<int64> IDs) = 0;

	/**
	 * Search for K nearest neighbors. Continuation receives the results sorted
	 * by ascending distance. The continuation is owned (by-value TFunction) so
	 * implementations may move it into a deferred task and invoke it
	 * asynchronously off the calling thread.
	 *
	 * @param IDFilter        Shared, immutable list of IDs to restrict the
	 *                        search to. Empty array means "no filter".
	 * @param DistanceCutoff  Drop results whose distance is >= this value before
	 *                        returning. Pass TNumericLimits<float>::Max() for "no cutoff".
	 *                        Lets callers push their distance threshold down so
	 *                        the index can avoid materializing results that would
	 *                        be discarded downstream anyway.
	 * @param Scratch         Caller-owned scratch buffer that can be reused to speed up some algo
	 * @param IndexReadCompleteEvent
	 *                        Optional FGraphEventRef the implementation must
	 *                        DispatchSubsequents() on after its index-touching
	 *                        phase finishes.
	 * @param Continuation    Invoked exactly once with the result array.
	 */
	virtual void Search(
		TConstArrayView<float> QueryVector,
		int32 K,
		const TSharedRef<const TArray<int64>>& IDFilter,
		float DistanceCutoff,
		TArray<uint32>& Scratch,
		FGraphEventRef IndexReadCompleteEvent,
		TFunction<void(TArray<FSearchResult>&&)> Continuation) const = 0;

	/** Number of vectors currently in the index. */
	virtual int64 GetCount() const = 0;

	/** Embedding dimension. */
	virtual int32 GetDimension() const = 0;

	/** Serialize the full index to a byte array. */
	virtual TArray<uint8> Serialize() const = 0;

	/** Whether this index supports quantization. If true, can static_cast to IQuantizedVectorIndex. */
	virtual bool SupportsQuantization() const { return false; }

	// --- Inline single-item helpers ---

	void Add(int64 ID, TConstArrayView<float> Vector)
	{
		Add(TConstArrayView<int64>(&ID, 1), Vector);
	}

	bool Remove(int64 ID)
	{
		const int64 CountBefore = GetCount();
		Remove(TConstArrayView<int64>(&ID, 1));
		return GetCount() < CountBefore;
	}


	/** Returns true if a vector with the given ID exists in the index. */
	virtual bool Contains(int64 ID) const { return false; }

	/** Retrieve the embedding for a given ID. Returns false if not supported or ID not found. */
	virtual bool TryGetEmbedding(int64 ID, TArray<float>& OutEmbedding) const { return false; }

	/** Estimated memory usage in bytes. Returns 0 if not implemented. */
	virtual int64 EstimateMemoryBytes() const { return 0; }

	/**
	 * Re-key a vector from OldID to NewID, preserving the embedding data.
	 * Returns true if the vector was found and updated, false if OldID was not in the index.
	 */
	virtual bool Update(int64 OldID, int64 NewID)
	{
		TArray<float> Embedding;
		if (!TryGetEmbedding(OldID, Embedding))
		{
			return false;
		}
		Remove(OldID);
		Add(NewID, Embedding);
		return true;
	}
};

} // namespace UE::SemanticSearch
