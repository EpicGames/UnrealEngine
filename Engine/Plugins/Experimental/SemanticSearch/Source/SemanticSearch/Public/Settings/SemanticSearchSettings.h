// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "SemanticSearchSettings.generated.h"

UENUM()
enum class ESemanticSearchIndexType : uint8
{
	Flat    UMETA(DisplayName = "Flat (Exact)"),
	PQ      UMETA(DisplayName = "PQ (Compressed)")
};

UENUM()
enum class ESemanticSearchEmbeddingProvider : uint8
{
	OpenAI  UMETA(DisplayName = "OpenAI-compatible API")
};

UCLASS(config = EditorPerProjectUserSettings, DefaultConfig, MinimalAPI, meta = (DisplayName = "Semantic Search"))
class USemanticSearchSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static const USemanticSearchSettings* Get()
	{
		return GetDefault<USemanticSearchSettings>();
	}

	virtual FName GetContainerName() const override { return TEXT("Editor"); }
	virtual FName GetCategoryName() const override { return TEXT("Plugins"); }
	virtual FName GetSectionName() const override { return TEXT("Semantic Search"); }

	SEMANTICSEARCH_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UPROPERTY(EditAnywhere, Config, Category = "Indexing",
		meta = (ToolTip = "Content browser paths to include in the semantic search index.",
			ContentDir, LongPackageName, ForceShowEngineContent, ForceShowPluginContent))
	TArray<FDirectoryPath> IndexedPaths = { {TEXT("/")} };

	UPROPERTY(EditAnywhere, Config, Category = "Indexing",
		meta = (ToolTip = "When true, automatically loads cached embeddings from DDC on editor startup, avoiding the need to manually refresh the index."))
	bool bAutoIndexCachedOnStartup = false;

	UPROPERTY(EditAnywhere, Config, Category = "Indexing",
		meta = (ToolTip = "When true, assets without cached embeddings are sent to the backend during auto-indexing on startup. When false, only DDC-cached assets are loaded. Ignored if Auto Index Cached On Startup is disabled. WARNING: This can be slow for large projects with many uncached assets."))
	bool bAutoIndexUncachedOnStartup = false;

	UPROPERTY(EditAnywhere, Config, Category = "Indexing",
		meta = (ToolTip = "When true, assets are automatically re-indexed when saved or updated on disk."))
	bool bAutoIndexOnSave = true;

	UPROPERTY(EditAnywhere, Config, Category = "Provider",
		meta = (ToolTip = "Which embedding/captioning backend to use. Changes take effect immediately; the vector index is rebuilt as Flat at the new dimension."))
	ESemanticSearchEmbeddingProvider Provider = ESemanticSearchEmbeddingProvider::OpenAI;

	UPROPERTY(EditAnywhere, Config, Category = "Provider",
		meta = (EditCondition = "Provider == ESemanticSearchEmbeddingProvider::OpenAI",
				EditConditionHides,
				ToolTip = "Base URL for the captioning endpoint. Provider appends '/chat/completions'. Defaults to OpenAI; can point at any OpenAI-compatible server like LiteLLM or Ollama."))
	FString CaptioningBaseUrl = TEXT("https://api.openai.com/v1");

	UPROPERTY(EditAnywhere, Config, Category = "Provider",
		meta = (EditCondition = "Provider == ESemanticSearchEmbeddingProvider::OpenAI",
				EditConditionHides,
				PasswordField,
				ToolTip = "API key for the captioning endpoint. Sent as 'Authorization: Bearer <key>'."))
	FString CaptioningApiKey;

	UPROPERTY(EditAnywhere, Config, Category = "Provider",
		meta = (EditCondition = "Provider == ESemanticSearchEmbeddingProvider::OpenAI",
				EditConditionHides,
				ToolTip = "Vision-capable chat completions model used to generate tags + caption for an asset."))
	FString CaptioningModel = TEXT("gpt-5.4-mini");

	UPROPERTY(EditAnywhere, Config, Category = "Provider",
		meta = (EditCondition = "Provider == ESemanticSearchEmbeddingProvider::OpenAI",
				EditConditionHides,
				ToolTip = "Base URL for the embedding endpoint. Provider appends '/embeddings'. Can differ from the captioning base URL (e.g., embeddings on a local server, captions on OpenAI)."))
	FString EmbeddingBaseUrl = TEXT("https://api.openai.com/v1");

	UPROPERTY(EditAnywhere, Config, Category = "Provider",
		meta = (EditCondition = "Provider == ESemanticSearchEmbeddingProvider::OpenAI",
				EditConditionHides,
				PasswordField,
				ToolTip = "API key for the embedding endpoint. Sent as 'Authorization: Bearer <key>'."))
	FString EmbeddingApiKey;

	UPROPERTY(EditAnywhere, Config, Category = "Provider",
		meta = (EditCondition = "Provider == ESemanticSearchEmbeddingProvider::OpenAI",
				EditConditionHides,
				ToolTip = "Embedding model used to embed the asset caption returned from the captioning model."))
	FString EmbeddingModel = TEXT("text-embedding-3-small");

	UPROPERTY(EditAnywhere, Config, Category = "Provider",
		meta = (EditCondition = "Provider == ESemanticSearchEmbeddingProvider::OpenAI",
				EditConditionHides,
				ClampMin = "1", ClampMax = "3072",
				ToolTip = "Embedding vector dimension. Sent as the 'dimensions' parameter to OpenAI embeddings endpoint."))
	int32 EmbeddingDimension = 512;

	UPROPERTY(EditAnywhere, Config, Category = "HTTP",
		meta = (ClampMin = "0", ClampMax = "10",
				ToolTip = "Number of times to retry an HTTP request on a retryable failure (429, 5xx)."))
	int32 MaxRetries = 1;

	UPROPERTY(EditAnywhere, Config, Category = "HTTP",
		meta = (ClampMin = "1.0", ClampMax = "3600.0",
				ToolTip = "Maximum total wall-clock seconds the retry manager will keep retrying a single request before giving up."))
	float RetryTimeoutSeconds = 300.0f;

	UPROPERTY(EditAnywhere, Config, Category = "HTTP",
		meta = (ClampMin = "1", ClampMax = "10000",
				ToolTip = "Maximum concurrent caption HTTP requests in flight. Excess requests are queued."))
	int32 MaxConcurrentCaptionRequests = 256;

	UPROPERTY(EditAnywhere, Config, Category = "HTTP",
		meta = (ClampMin = "1", ClampMax = "10000",
				ToolTip = "Maximum concurrent embedding HTTP requests in flight. Excess requests are queued."))
	int32 MaxConcurrentEmbedRequests = 256;

	UPROPERTY(VisibleAnywhere, Config, Category = "Vector Index",
		meta = (ToolTip = "Index type is managed via Tools > Semantic Search."))
	ESemanticSearchIndexType IndexType = ESemanticSearchIndexType::PQ;

	UPROPERTY(EditAnywhere, Config, Category = "Vector Index",
		meta = (EditCondition = "IndexType == ESemanticSearchIndexType::PQ", EditConditionHides,
			ClampMin = "1", ClampMax = "64",
			ToolTip = "Number of dimensions per subquantizer. Embedding dimension must be divisible by this."))
	int32 PQSubvectorSize = 8;

	UPROPERTY(EditAnywhere, Config, Category = "Vector Index",
		meta = (EditCondition = "IndexType == ESemanticSearchIndexType::PQ", EditConditionHides,
			ClampMin = "4", ClampMax = "16",
			ToolTip = "Bits per subquantizer code. 8 = 256 centroids per subvector."))
	int32 PQNBits = 8;

	UPROPERTY(EditAnywhere, Config, Category = "Search")
	bool bEnableHybridSearch = true;

	UPROPERTY(EditAnywhere, Config, Category = "Search",
		meta = (EditCondition = "bEnableHybridSearch", EditConditionHides,
			ClampMin = "1", ClampMax = "1000",
			ToolTip = "RRF constant (k). Higher values reduce the impact of rank differences."))
	int32 RRFConstant = 60;

	UPROPERTY(EditAnywhere, Config, Category = "Search",
		meta = (EditCondition = "bEnableHybridSearch", EditConditionHides,
			ClampMin = "1", ClampMax = "10",
			ToolTip = "Oversample multiplier for RRF. Each source fetches K * this many candidates before fusion."))
	int32 RRFOversample = 3;

	UPROPERTY(EditAnywhere, Config, Category = "Search",
		meta = (EditCondition = "bEnableHybridSearch", EditConditionHides,
			ClampMin = "0", ClampMax = "100",
			ToolTip = "Minimum percentage of unique query terms a document must match to be returned by BM25. 0 = any term (pure OR), 100 = all terms (strict AND). Default 75 prevents single-term matches like 'green' from polluting results for queries like 'green chair'. Computed via ceiling, e.g. 75% of 4 terms requires 3."))
	int32 BM25MinShouldMatchPercent = 75;

	UPROPERTY(EditAnywhere, Config, Category = "Search",
		meta = (ClampMin = "0.1", ClampMax = "3.0",
			ToolTip = "Maximum L2 distance for a result to be considered a semantic match. Lower values are stricter. Results must pass this threshold OR have a BM25 keyword match to appear."))
	float VectorDistanceCutoff = 1.0f;

	UPROPERTY(EditAnywhere, Config, Category = "Search",
		meta = (ClampMin = "0.1", ClampMax = "3.0",
			ToolTip = "Maximum L2 distance for 'Find Similar' results. Lower values are stricter."))
	float SimilarityDistanceCutoff = 0.5f;
};
