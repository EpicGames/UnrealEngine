// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

/**
 * Core module for semantic search service
 * Provide the key system for the queries and management of the generation of the embeddings
 *
 * Any ui or higher level logic should go in another module
 */
public class SemanticSearch : ModuleRules
{
	public SemanticSearch(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"Engine",
				"HTTP",
				"ImageCore",
				"Json",
				"PhysicsCore",
				"Projects",
				"Slate",
				"UnrealEd",
				"DeveloperSettings",
				"FAISS"
			});

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"DerivedDataCache",
			});
	}
}