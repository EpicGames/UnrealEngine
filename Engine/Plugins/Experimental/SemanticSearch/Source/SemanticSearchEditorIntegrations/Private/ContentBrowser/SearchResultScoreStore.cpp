// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowser/SearchResultScoreStore.h"

#include "AssetRegistry/AssetData.h"
#include "ContentBrowserModule.h"
#include "HybridSearchIndex.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::SemanticSearch::ContentBrowser
{

FSearchResultScoreStore& FSearchResultScoreStore::Get()
{
	static FSearchResultScoreStore Instance;
	return Instance;
}

void FSearchResultScoreStore::Set(TArray<int64>&& InEntryIDs, TArray<FAssetScores>&& InEntryScores, FHashTable&& InIDHashTable)
{
	check(InEntryIDs.Num() == InEntryScores.Num());

	EntryIDs = MoveTemp(InEntryIDs);
	EntryScores = MoveTemp(InEntryScores);
	IDHashTable = MoveTemp(InIDHashTable);
}

void FSearchResultScoreStore::Clear()
{
	EntryIDs.Reset();
	EntryScores.Reset();
	// Reset to default-empty FHashTable. Move-assign from a temporary releases
	// the old buffers; cheaper and simpler than reaching for Free() directly.
	IDHashTable = FHashTable();
}

void FSearchResultScoreStore::UpdateMetadata(int64 AssetID, FString&& Caption, TArray<FString>&& Keywords)
{
	const uint32 IDHash = (uint32)MurmurFinalize64((uint64)AssetID);
	for (uint32 idx = IDHashTable.First(IDHash); IDHashTable.IsValid(idx); idx = IDHashTable.Next(idx))
	{
		if (EntryIDs[idx] == AssetID)
		{
			EntryScores[idx].Caption = MoveTemp(Caption);
			EntryScores[idx].Keywords = MoveTemp(Keywords);
			return;
		}
	}
}

const FAssetScores* FSearchResultScoreStore::Find(int64 AssetID) const
{
	const uint32 IDHash = (uint32)MurmurFinalize64((uint64)AssetID);
	for (uint32 idx = IDHashTable.First(IDHash); IDHashTable.IsValid(idx); idx = IDHashTable.Next(idx))
	{
		if (EntryIDs[idx] == AssetID)
		{
			return &EntryScores[idx];
		}
	}
	return nullptr;
}

namespace
{
	static FDelegateHandle GScoreBadgeHandle;

	// CB tooltip path renders IconGenerator alongside ToolTipGenerator with no opt-out.
	// We capture the most recent icon here so MakeTooltipWidget can collapse the
	// duplicate (it always runs immediately after the tooltip-context icon).
	static TWeakPtr<SWidget> GLastIconWidget;

	// Cosine from squared L2 of unit vectors: cos = 1 - d² / 2.
	float DistanceToSimilarity(float Distance)
	{
		return FMath::Clamp(1.0f - Distance * 0.5f, 0.0f, 1.0f);
	}

	FLinearColor SimilarityToColor(float Similarity)
	{
		// Stops tuned to the empirical cosine distribution, not the theoretical [0,1].
		constexpr float RedStop   = 0.40f;
		constexpr float AmberStop = 0.60f;
		constexpr float GreenStop = 0.80f;

		const FLinearColor Red   = FLinearColor(0.90f, 0.25f, 0.20f);
		const FLinearColor Amber = FLinearColor(0.95f, 0.70f, 0.15f);
		const FLinearColor Green = FLinearColor(0.30f, 0.85f, 0.25f);

		if (Similarity <= RedStop)
		{
			return Red;
		}
		if (Similarity >= GreenStop)
		{
			return Green;
		}
		if (Similarity <= AmberStop)
		{
			const float T = (Similarity - RedStop) / (AmberStop - RedStop);
			return FLinearColor::LerpUsingHSV(Red, Amber, T);
		}
		const float T = (Similarity - AmberStop) / (GreenStop - AmberStop);
		return FLinearColor::LerpUsingHSV(Amber, Green, T);
	}

	TSharedRef<SWidget> MakeIconWidget(const FAssetData& AssetData)
	{
		const int64 AssetID = GetAssetIndexID(AssetData);

		auto VectorVis = [AssetID]() -> EVisibility
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			return (Scores && Scores->VectorDistance >= 0.0f) ? EVisibility::Visible : EVisibility::Collapsed;
		};

		auto VectorText = [AssetID]() -> FText
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			if (!Scores || Scores->VectorDistance < 0.0f)
			{
				return FText::GetEmpty();
			}
			return FText::AsPercent(DistanceToSimilarity(Scores->VectorDistance));
		};

		auto VectorColor = [AssetID]() -> FSlateColor
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			if (!Scores || Scores->VectorDistance < 0.0f)
			{
				return FSlateColor::UseForeground();
			}
			return FSlateColor(SimilarityToColor(DistanceToSimilarity(Scores->VectorDistance)));
		};

		auto BM25Vis = [AssetID]() -> EVisibility
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			return (Scores && Scores->BM25Score > 0.0f) ? EVisibility::Visible : EVisibility::Collapsed;
		};

		// Dark semi-transparent pill so colored text reads on light thumbnails.
		const FSlateBrush* PillBrush = FCoreStyle::Get().GetBrush("WhiteBrush");

		GLastIconWidget.Reset();

		TSharedRef<SBorder> Badge = SNew(SBorder)
			.BorderImage(PillBrush)
			.BorderBackgroundColor(FLinearColor(0.f, 0.f, 0.f, 0.55f))
			.Padding(FMargin(3.f, 1.f))
			.Visibility_Lambda([AssetID]() -> EVisibility
			{
				const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
				const bool bAnyHit = Scores && (Scores->VectorDistance >= 0.0f || Scores->BM25Score > 0.0f);
				return bAnyHit ? EVisibility::Visible : EVisibility::Collapsed;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "TinyText")
					.Visibility_Lambda(VectorVis)
					.Text_Lambda(VectorText)
					.ColorAndOpacity_Lambda(VectorColor)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "TinyText")
					.Visibility_Lambda(BM25Vis)
					.Text(NSLOCTEXT("SemanticSearch", "BM25Badge", "T"))
					.ColorAndOpacity(FSlateColor(FLinearColor(0.30f, 0.75f, 1.0f)))
					.ToolTipText(NSLOCTEXT("SemanticSearch", "BM25BadgeTooltip", "Keyword match (BM25)"))
				]
			];

		GLastIconWidget = Badge;
		return Badge;
	}

	TSharedRef<SWidget> MakeTooltipWidget(const FAssetData& AssetData)
	{
		// Hide the duplicate badge the CB forced alongside this tooltip — see GLastIconWidget.
		if (TSharedPtr<SWidget> PrevIcon = GLastIconWidget.Pin())
		{
			PrevIcon->SetVisibility(EVisibility::Collapsed);
			GLastIconWidget.Reset();
		}

		const int64 AssetID = GetAssetIndexID(AssetData);

		auto Vis = [AssetID]() -> EVisibility
		{
			return FSearchResultScoreStore::Get().Find(AssetID) ? EVisibility::Visible : EVisibility::Collapsed;
		};

		auto MetricsText = [AssetID]() -> FText
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			if (!Scores)
			{
				return FText::GetEmpty();
			}
			TArray<FString> Parts;
			if (Scores->VectorDistance >= 0.0f)
			{
				const float Sim = DistanceToSimilarity(Scores->VectorDistance);
				Parts.Add(FText::AsPercent(Sim).ToString());
				Parts.Add(FString::Printf(TEXT("d² %.2f"), Scores->VectorDistance));
			}
			if (Scores->BM25Score > 0.0f)
			{
				Parts.Add(FString::Printf(TEXT("BM25 %.2f"), Scores->BM25Score));
			}
			return FText::FromString(FString::Join(Parts, TEXT("  ·  ")));
		};

		auto CaptionVis = [AssetID]() -> EVisibility
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			return (Scores && !Scores->Caption.IsEmpty()) ? EVisibility::Visible : EVisibility::Collapsed;
		};

		auto CaptionText = [AssetID]() -> FText
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			return (Scores && !Scores->Caption.IsEmpty())
				? FText::FromString(Scores->Caption)
				: FText::GetEmpty();
		};

		auto KeywordsVis = [AssetID]() -> EVisibility
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			return (Scores && Scores->Keywords.Num() > 0) ? EVisibility::Visible : EVisibility::Collapsed;
		};

		auto KeywordsText = [AssetID]() -> FText
		{
			const FAssetScores* Scores = FSearchResultScoreStore::Get().Find(AssetID);
			if (!Scores || Scores->Keywords.Num() == 0)
			{
				return FText::GetEmpty();
			}
			return FText::FromString(FString::Join(Scores->Keywords, TEXT(" · ")));
		};

		return SNew(SVerticalBox)
			.Visibility_Lambda(Vis)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "TinyText")
				.Text_Lambda(MetricsText)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 6.f, 0.f, 0.f))
			[
				SNew(STextBlock)
				.Visibility_Lambda(CaptionVis)
				.Text_Lambda(CaptionText)
				.AutoWrapText(true)
				.WrapTextAt(340.f)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 6.f, 0.f, 0.f))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "TinyText")
				.Visibility_Lambda(KeywordsVis)
				.Text_Lambda(KeywordsText)
				.AutoWrapText(true)
				.WrapTextAt(340.f)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];
	}
}

void RegisterScoreBadgeGenerator()
{
	FContentBrowserModule& CBModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	if (GScoreBadgeHandle.IsValid())
	{
		CBModule.RemoveAssetViewExtraStateGenerator(GScoreBadgeHandle);
		GScoreBadgeHandle.Reset();
	}

	FAssetViewExtraStateGenerator Generator(
		FOnGenerateAssetViewExtraStateIndicators::CreateStatic(&MakeIconWidget),
		FOnGenerateAssetViewExtraStateIndicators::CreateStatic(&MakeTooltipWidget));
	GScoreBadgeHandle = CBModule.AddAssetViewExtraStateGenerator(Generator);
}

void UnregisterScoreBadgeGenerator()
{
	if (!GScoreBadgeHandle.IsValid())
	{
		return;
	}
	if (FContentBrowserModule* CBModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		CBModule->RemoveAssetViewExtraStateGenerator(GScoreBadgeHandle);
	}
	GScoreBadgeHandle.Reset();
}

}
