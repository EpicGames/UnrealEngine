// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/HashTable.h"

namespace UE::SemanticSearch::ContentBrowser
{

struct FAssetScores
{
	float RRFScore = 0.0f;
	float VectorDistance = -1.0f;
	float BM25Score = -1.0f;
	FString Caption;
	TArray<FString> Keywords;
};

/**
 * Game-thread-only score store backing tooltip / badge display for content-
 * browser search results. No synchronization is needed — Slate and the search
 * override callbacks both run on the game thread.
 *
 * Storage is a parallel-array + FHashTable layout instead of TMap so the search
 * pipeline can build the payload on a worker thread (parallel ID writes +
 * FHashTable Add_Concurrent) and hand it off via a single move-assign on GT.
 */
class FSearchResultScoreStore
{
public:
	static FSearchResultScoreStore& Get();

	void Set(TArray<int64>&& InEntryIDs, TArray<FAssetScores>&& InEntryScores, FHashTable&& InIDHashTable);
	void Clear();

	/**
	 * Update the cached caption / keywords for an indexed asset. No-op if the
	 * ID is absent (e.g. the store was cleared by a newer search mid-fetch).
	 *
	 * @param AssetID   Asset identifier to update.
	 * @param Caption   New caption text. Moved into the store.
	 * @param Keywords  New keyword list. Moved into the store.
	 */
	void UpdateMetadata(int64 AssetID, FString&& Caption, TArray<FString>&& Keywords);

	const FAssetScores* Find(int64 AssetID) const;
	int32 Num() const { return EntryIDs.Num(); }

private:
	TArray<int64> EntryIDs;
	TArray<FAssetScores> EntryScores;
	FHashTable IDHashTable;
};

void RegisterScoreBadgeGenerator();
void UnregisterScoreBadgeGenerator();

}
