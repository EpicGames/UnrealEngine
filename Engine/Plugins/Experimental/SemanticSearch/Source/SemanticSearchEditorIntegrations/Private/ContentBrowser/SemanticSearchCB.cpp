// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowser/SemanticSearchCB.h"

#include "ContentBrowser/SearchResultScoreStore.h"
#include "AnalyticsEventAttribute.h"
#include "AssetProcessorManager.h"
#include "AssetViewTypes.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/HashTable.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "EngineAnalytics.h"
#include "HAL/IConsoleManager.h"
#include "IContentBrowserSingleton.h"
#include "Experimental/ContentBrowserSearchOverride.h"
#include "HybridSearchIndex.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "ISemanticSearchModule.h"
#include "Settings/SemanticSearchSettings.h"
#include "Input/Reply.h"
#include "Interfaces/IEmbeddingProvider.h"
#include "Interfaces/IVectorIndex.h"
#include "Misc/DelayedAutoRegister.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "ToolMenus.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "AssetRegistry/IAssetRegistry.h"

#include <atomic>

DEFINE_LOG_CATEGORY_STATIC(LogSemanticSearchCB, Log, All);

namespace UE::SemanticSearch::ContentBrowser
{
	using namespace UE::Editor::ContentBrowser::Extension;
	using namespace UE::SemanticSearch;

	static bool bIsAIActive = false;

namespace Private
{

	static TAutoConsoleVariable<int32> CVarForceSequentialContentBrowserSearch(
		TEXT("SemanticSearch.ForceSequentialContentBrowserSearch"),
		0,
		TEXT("Profiling aid. When non-zero, the AsyncTask background dispatches inside the content-browser semantic-search path run inline on the calling thread instead of being scheduled. ")
		TEXT("Combine with SemanticSearch.ForceSequentialSemanticSearchIndexWorker=1 (and optionally SemanticSearch.KeepParallelFor=0) for an end-to-end flat trace. Default: 0."),
		ECVF_Default);

	static bool ShouldForceSequentialContentBrowserSearch()
	{
		return CVarForceSequentialContentBrowserSearch.GetValueOnAnyThread() != 0;
	}

	template <typename FuncType>
	static void DispatchBackgroundOrInline(FuncType&& Func)
	{
		if (ShouldForceSequentialContentBrowserSearch())
		{
			Func();
		}
		else
		{
			AsyncTask(ENamedThreads::AnyBackgroundHiPriTask, Forward<FuncType>(Func));
		}
	}

	// ---------------------------------------------------------------------------

	class FSemanticSearchAssetViewOverride : public ICBSearchOverrideInstance, public TSharedFromThis<FSemanticSearchAssetViewOverride>
	{
	public:
		virtual TSharedRef<SWidget> GetOverrideModeToggleWidget() override;

		virtual bool HasPendingResults() const override { return bHasInFlightRequest; }

		virtual void OnSearchTextChanged(const FText& NewText) override {}

		virtual void OnSearchTextCommitted(const FText& NewText, ETextCommit::Type) override
		{
			if (NewText.EqualTo(ActiveQuery))
			{
				return;
			}

			RequestQuickFrontendListRefresh();
			ActiveQuery = NewText;
		}

		virtual void OnItemsAvailable(TArrayView<const TSharedPtr<FAssetViewItem>> Items) override
		{
			CurrentItems.Append(Items);

			// When the search completed with zero matches, the SearchAsync callback
			// left bHasInFlightRequest true so the CB keeps ticking.  Finalize here
			// so the amortization counter can reset on this tick.
			if (bHasActiveResults && bHasInFlightRequest)
			{
				bHasInFlightRequest = false;
				UE_LOGF(LogSemanticSearchCB, Verbose, "OnItemsAvailable: finalizing in-flight request (Epoch=%u, Query='%ls', IncomingItems=%d, CurrentItems=%d, LastMatched=%d)",
					ActiveEpoch, *ActiveQuery.ToString(), Items.Num(), CurrentItems.Num(), LastMatchedItems.Num());
				PublishMatchingItems();
				return;
			}

			if (Items.IsEmpty())
			{
				return;
			}

			if (ActiveQuery.IsEmpty())
			{
				UE_LOGF(LogSemanticSearchCB, Verbose, "OnItemsAvailable: empty query passing items through unfiltered (Epoch=%u, IncomingItems=%d, CurrentItems=%d)",
					ActiveEpoch, Items.Num(), CurrentItems.Num());
				// In AI mode with no query, surface all asset-view items (including unindexed ones)
				// so the user can navigate folders normally instead of seeing only indexed assets.
				TArray<TSharedPtr<FAssetViewItem>> Passthrough(Items.GetData(), Items.Num());
				PublishResults(Passthrough);
			}
			else if (bHasActiveResults)
			{
				UE_LOGF(LogSemanticSearchCB, Verbose, "OnItemsAvailable: re-publishing matching items for late-arriving batch (Epoch=%u, Query='%ls', IncomingItems=%d, LastMatched=%d)",
					ActiveEpoch, *ActiveQuery.ToString(), Items.Num(), LastMatchedItems.Num());
				PublishMatchingItems();
			}
		}

		virtual bool IsSortOverridden() const override { return !ActiveQuery.IsEmpty(); }

		virtual void SortItemList(TArrayView<TSharedPtr<FAssetViewItem>> Items) override
		{

		}

		// Gate hybrid SearchAsync on receiving the full item set: running the
		// vector/BM25 query before OnItemsAvailable has delivered everything would
		// filter against a partial IDFilter and permanently drop the unseen assets
		// from the ranking.  We flip the flag here and drain any embedding that
		// raced ahead of it.
		virtual void OnAllKnownItemsAvailableImplementation() override
		{
			UE_LOGF(LogSemanticSearchCB, Verbose, "OnAllKnownItemsAvailable (Epoch=%u, Query='%ls', CurrentItems=%d, PendingEmbedding=%ls)",
				ActiveEpoch, *ActiveQuery.ToString(), CurrentItems.Num(), PendingQueryEmbedding.IsSet() ? TEXT("yes") : TEXT("no"));
			bAllKnownItemsAvailable = true;

			if (!PendingQueryEmbedding.IsSet())
			{
				return;
			}

			TArray<float> Embedding = MoveTemp(*PendingQueryEmbedding);
			const uint32 Epoch = PendingQueryEmbeddingEpoch;
			PendingQueryEmbedding.Reset();
			DispatchHybridSearch(MoveTemp(Embedding), Epoch);
		}

	protected:

		// Incrementing the epoch silently invalidates all in-flight callbacks:
		// every async job captures the epoch at launch and bails if it has changed.
		// All epoch reads/writes happen inside AsyncTask(GameThread), so the game
		// thread is the sole synchronization point — no atomics required.
		virtual void OnFilteringResetImplementation() override
		{
			UE_LOGF(LogSemanticSearchCB, Verbose, "Filtering reset — invalidating in-flight async work (OldEpoch=%u, NewEpoch=%u, Query='%ls', CurrentItems=%d, LastMatched=%d, HadInFlight=%ls, HadActiveResults=%ls)",
				ActiveEpoch, ActiveEpoch + 1, *ActiveQuery.ToString(), CurrentItems.Num(), LastMatchedItems.Num(),
				bHasInFlightRequest ? TEXT("yes") : TEXT("no"), bHasActiveResults ? TEXT("yes") : TEXT("no"));
			++ActiveEpoch;
			bHasInFlightRequest = false;
			CurrentItems.Reset();
			LastMatchedItems.Reset();
			bHasActiveResults = false;
			bAllKnownItemsAvailable = false;
			PendingQueryEmbedding.Reset();
			FSearchResultScoreStore::Get().Clear();

			if (ActiveQuery.IsEmpty())
			{
				SetUserSearching(false);
				return;
			}
			else
			{
				SetUserSearching(true);
				SubmitQueryAsync(ActiveQuery.ToString());
			}
		}

	private:

		void HandleFindSimilar(const FString& AssetPath)
		{
			FSemanticSearchIndexStats Stats = FHybridSearchIndex::Get().GetCachedIndexStats();
			if (Stats.VectorCount == 0)
			{
				UE_LOGF(LogSemanticSearchCB, Warning, "Find Similar: index is empty, nothing to compare against (%ls)", *AssetPath);
				FinalizeEmptyResults();
				return;
			}

			const int64 AssetID = GetAssetIndexID(AssetPath);

			++ActiveEpoch;
			bHasInFlightRequest = true;
			bIsSimilaritySearch = true;
			const uint32 Epoch = ActiveEpoch;

			TWeakPtr<FSemanticSearchAssetViewOverride> WeakSelf = AsShared();
			FHybridSearchIndex::Get().ExtractEmbeddingsAsync({ AssetID },
				[WeakSelf, Epoch, AssetPath](TArray<int64>&& IDs, TArray<float>&& Vectors, int32 Dimension) mutable
				{
					TSharedPtr<FSemanticSearchAssetViewOverride> Self = WeakSelf.Pin();
					if (!Self || Self->ActiveEpoch != Epoch)
					{
						return;
					}

					if (IDs.IsEmpty() || Vectors.IsEmpty())
					{
						UE_LOGF(LogSemanticSearchCB, Warning, "Find Similar: '%ls' is not in the semantic index — returning no results", *AssetPath);
						Self->FinalizeEmptyResults();
						return;
					}


					UE_LOGF(LogSemanticSearchCB, Verbose, "Find Similar: extracted embedding for '%ls' (Epoch=%u, Dim=%d, VectorCount=%d)",
						*AssetPath, Epoch, Dimension, IDs.Num());
					TArray<float> Embedding(Vectors.GetData(), Dimension);

					// PQ reconstruction is lossy: renormalize so the find-similar query
					// matches the text-query path (which the provider returns at ||v||=1).
					float NormSq = 0.0f;
					for (float V : Embedding) { NormSq += V * V; }
					if (NormSq > UE_SMALL_NUMBER)
					{
						const float InvNorm = FMath::InvSqrt(NormSq);
						for (float& V : Embedding) { V *= InvNorm; }
					}

					Self->DispatchHybridSearch(MoveTemp(Embedding), Epoch);
				});
		}

		// Route a "no matches" outcome through the same path the SearchAsync callback
		// uses for zero-match hybrid searches: keep bHasInFlightRequest=true and mark
		// results active so OnItemsAvailable finalizes on the next tick.  Skipping
		// this leaves the CB stuck on "Applying filter…" forever.
		void FinalizeEmptyResults()
		{
			LastMatchedItems.Reset();
			bHasActiveResults = true;
			bHasInFlightRequest = true;
			FSearchResultScoreStore::Get().Clear();
		}

		void SubmitQueryAsync(FString QueryText)
		{
			// Parse slash commands
			static const FString FindSimilarPrefix = TEXT("/find-similar ");
			const bool bIsFindSimilar = QueryText.StartsWith(FindSimilarPrefix);

			if (FEngineAnalytics::IsAvailable())
			{
				TArray<FAnalyticsEventAttribute> Attrs;
				Attrs.Emplace(TEXT("Type"), bIsFindSimilar ? TEXT("FindSimilar") : TEXT("Text"));
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.SemanticSearch.Query"), Attrs);
			}

			if (bIsFindSimilar)
			{
				HandleFindSimilar(QueryText.RightChop(FindSimilarPrefix.Len()));
				return;
			}

			bIsSimilaritySearch = false;
			TSharedPtr<IEmbeddingProvider> Provider = ISemanticSearchModule::Get().GetEmbeddingProvider();
			if (!Provider)
			{
				UE_LOGF(LogSemanticSearchCB, Warning, "No embedding provider registered.");
				return;
			}

			++ActiveEpoch;
			bHasInFlightRequest = true;
			const uint32 Epoch = ActiveEpoch;

			TWeakPtr<FSemanticSearchAssetViewOverride> WeakSelf = AsShared();
			Provider->GenerateEmbeddingAsync(QueryText, [WeakSelf, Epoch](FEmbeddingResponse&& Response) mutable
			{
				// Provider callback may arrive on any thread; marshal to GT before
				// touching any shared state.
				AsyncTask(ENamedThreads::GameThread, [WeakSelf, Epoch, Response = MoveTemp(Response)]() mutable
				{
					TSharedPtr<FSemanticSearchAssetViewOverride> Self = WeakSelf.Pin();
					if (!Self || Self->ActiveEpoch != Epoch)
					{
						return;
					}
					if (Response.ErrorMessage.IsEmpty())
					{
						UE_LOGF(LogSemanticSearchCB, Verbose, "Query embedding generated (Epoch=%u, Query='%ls', EmbeddingSize=%d)",
							Epoch, *Self->ActiveQuery.ToString(), Response.Embedding.Num());
						Self->DispatchHybridSearch(MoveTemp(Response.Embedding), Epoch);
					}
					else
					{
						UE_LOGF(LogSemanticSearchCB, Warning, "Query embedding failed: %ls", *Response.ErrorMessage);
						Self->bHasInFlightRequest = false;
					}
				});
			}, nullptr);
		}

		void DispatchHybridSearch(TArray<float>&& QueryEmbedding, uint32 Epoch)
		{
			if (ActiveEpoch != Epoch)
			{
				return;
			}

			// Hold the embedding until OnAllKnownItemsAvailableImplementation fires
			// — otherwise the IDFilter we build below only reflects the batches of
			// items that have arrived so far, and SearchAsync will exclude the
			// rest.  The drain happens in OnAllKnownItemsAvailableImplementation.
			if (!bAllKnownItemsAvailable)
			{
				PendingQueryEmbedding = MoveTemp(QueryEmbedding);
				PendingQueryEmbeddingEpoch = Epoch;
				return;
			}

			UE_LOGF(LogSemanticSearchCB, Verbose, "DispatchHybridSearch: running hybrid search (Epoch=%u, Query='%ls', CurrentItems=%d, Similarity=%ls)",
				Epoch, *ActiveQuery.ToString(), CurrentItems.Num(), bIsSimilaritySearch ? TEXT("yes") : TEXT("no"));
			if (CurrentItems.IsEmpty())
			{
				bHasInFlightRequest = false;
				return;
			}

			const double DispatchStartSeconds = FPlatformTime::Seconds();
			// Build ItemPaths and ItemPtrs on the game thread. 
			// Content browser data sourceaccess and CurrentItems iteration both assume GT ownership but blocking the GT while quering them is fine
			const int32 NumItems = CurrentItems.Num();
			TArray<FString> ItemPaths;
			ItemPaths.SetNum(NumItems);
			TArray<TSharedPtr<FAssetViewItem>> ItemPtrs;
			ItemPtrs.SetNum(NumItems);

			constexpr int32 MinBatchSize = 8192;
			ParallelFor(TEXT("SemanticSearchCB_BuildIDFilter"), NumItems, MinBatchSize,
				[this, &ItemPaths, &ItemPtrs](int32 Index)
				{
					const TSharedPtr<FAssetViewItem>& Item = CurrentItems[Index];
					if (!Item.IsValid())
					{
						return;
					}
					const FContentBrowserItem& CBItem = Item->GetItem();
					if (!CBItem.IsFile())
					{
						return;
					}
					FString Path;
					if (!CBItem.AppendItemObjectPath(Path))
					{
						return;
					}
					ItemPaths[Index] = MoveTemp(Path);
					ItemPtrs[Index] = Item;
				});

			// UObject and FText reads must happen on the game thread.
			// Everything downstream runs on a worker.
			FString QueryText = bIsSimilaritySearch ? FString() : ActiveQuery.ToString();
			TWeakPtr<FSemanticSearchAssetViewOverride> WeakSelf = AsShared();
			const bool bSimilarity = bIsSimilaritySearch;
			const USemanticSearchSettings* Settings = USemanticSearchSettings::Get();
			const float DistanceCutoff = bSimilarity
				? (Settings ? Settings->SimilarityDistanceCutoff : 0.8f)
				: (Settings ? Settings->VectorDistanceCutoff : 1.25f);

			DispatchBackgroundOrInline(
				[WeakSelf, Epoch, bSimilarity, DistanceCutoff, NumItems, DispatchStartSeconds,
				 QueryText = MoveTemp(QueryText),
				 QueryEmbedding = MoveTemp(QueryEmbedding),
				 ItemPaths = MoveTemp(ItemPaths),
				 ItemPtrs = MoveTemp(ItemPtrs)]() mutable
				{
					constexpr int32 BuildMinBatchSize = 8192;

					TArray<int64> EntryIDs;
					TArray<TWeakPtr<FAssetViewItem>> EntryItems;
					EntryIDs.SetNumUninitialized(NumItems);
					EntryItems.SetNum(NumItems);

					const uint32 HashSize = NumItems > 0
						? FMath::Max<uint32>(32u, FMath::RoundUpToPowerOfTwo((uint32)NumItems))
						: 32u;
					FHashTable IDHashTable(HashSize, (uint32)NumItems);

					std::atomic<int32> NextSlot{0};
					ParallelFor(TEXT("SemanticSearchCB_BuildEntries"), NumItems, BuildMinBatchSize,
						[&ItemPaths, &ItemPtrs, &EntryIDs, &EntryItems, &IDHashTable, &NextSlot](int32 i)
						{
							if (ItemPaths[i].IsEmpty())
							{
								return;
							}
							const int64 ID = GetAssetIndexID(ItemPaths[i]);
							const int32 Slot = NextSlot.fetch_add(1, std::memory_order_relaxed);
							EntryIDs[Slot] = ID;
							EntryItems[Slot] = ItemPtrs[i];
							IDHashTable.Add_Concurrent((uint32)MurmurFinalize64((uint64)ID), (uint32)Slot);
						});

					const int32 NumEntries = NextSlot.load(std::memory_order_relaxed);
					EntryIDs.SetNum(NumEntries, EAllowShrinking::No);
					EntryItems.SetNum(NumEntries, EAllowShrinking::No);

					// Empty QueryText on similarity search forces vector-only ranking (skips BM25).
					const int32 K = EntryIDs.Num();
					TConstArrayView<int64> IDFilterView(EntryIDs);
					const double SearchAsyncStartSeconds = FPlatformTime::Seconds();
					ISemanticSearchModule::Get().SearchAsync(
						QueryText, QueryEmbedding, K, IDFilterView, DistanceCutoff,
						[WeakSelf, Epoch, bSimilarity, DistanceCutoff, DispatchStartSeconds, SearchAsyncStartSeconds,
						 EntryIDs = MoveTemp(EntryIDs), EntryItems = MoveTemp(EntryItems), IDHashTable = MoveTemp(IDHashTable)](TArray<FHybridSearchResult>&& Results) mutable
						{
							const double SearchAsyncElapsedMs = (FPlatformTime::Seconds() - SearchAsyncStartSeconds) * 1000.0;
							UE_LOGF(LogSemanticSearchCB, Verbose, "SearchAsync round-trip latency %.4f ms (Epoch=%u, Results=%d)",
								SearchAsyncElapsedMs, Epoch, Results.Num());

							// Bail out if the search is cancel
							TSharedPtr<FSemanticSearchAssetViewOverride> Self = WeakSelf.Pin();
							if (!Self || Self->ActiveEpoch != Epoch)
							{
								return;
							}

							DispatchBackgroundOrInline(
								[WeakSelf, Epoch, bSimilarity, DistanceCutoff, DispatchStartSeconds,
								 EntryIDs = MoveTemp(EntryIDs),
								 EntryItems = MoveTemp(EntryItems),
								 IDHashTable = MoveTemp(IDHashTable),
								 Results = MoveTemp(Results)]() mutable
								{
									struct FScoreEntry
									{
										TSharedPtr<FAssetViewItem> Item;
										int64 ID = -1;
										FAssetScores Scores;
										// Populated on the game thread after the parallel mapping pass —
										// Legacy_TryGetAssetData is GT-only. Default-constructed entries
										FAssetData AssetData;
									};

									const int32 NumResults = Results.Num();
									TArray<FScoreEntry> Scores;
									Scores.SetNum(NumResults);

									constexpr int32 ResultsMinBatchSize = 8192;
									ParallelFor(TEXT("SemanticSearchCB_MapResults"), NumResults, ResultsMinBatchSize,
										[&Results, &EntryIDs, &EntryItems, &IDHashTable, &Scores, bSimilarity, DistanceCutoff](int32 i)
										{
											const FHybridSearchResult& Result = Results[i];
											const bool bHasKeywordMatch = !bSimilarity && Result.BM25Score > 0.0f;
											const bool bHasSemanticMatch = Result.VectorDistance >= 0.0f;

											if (!bHasKeywordMatch && !bHasSemanticMatch)
											{
												return;
											}

											const uint32 ResultHash = (uint32)MurmurFinalize64((uint64)Result.ID);
											for (uint32 idx = IDHashTable.First(ResultHash); IDHashTable.IsValid(idx); idx = IDHashTable.Next(idx))
											{
												if (EntryIDs[idx] == Result.ID)
												{
													if (TSharedPtr<FAssetViewItem> Item = EntryItems[idx].Pin())
													{
														Scores[i] = FScoreEntry{
															MoveTemp(Item),
															Result.ID,
															{ Result.RRFScore, Result.VectorDistance, Result.BM25Score }
														};
													}
													break;
												}
											}
										});

									// Perserve the order of items
									Scores.RemoveAll([](const FScoreEntry& Entry)
										{
											return !Entry.Item.IsValid();
										});

									const int32 NumScoreEntries = Scores.Num();
									TArray<int64> ScoreEntryIDs;
									TArray<FAssetScores> ScoreEntryValues;
									ScoreEntryIDs.SetNum(NumScoreEntries);
									ScoreEntryValues.SetNum(NumScoreEntries);

									const uint32 ScoreHashSize = NumScoreEntries > 0
										? FMath::Max<uint32>(32u, FMath::RoundUpToPowerOfTwo((uint32)NumScoreEntries))
										: 32u;
									FHashTable ScoreHashTable(ScoreHashSize, (uint32)NumScoreEntries);

									constexpr int32 ScoreBuildMinBatchSize = 8192;
									ParallelFor(TEXT("SemanticSearchCB_BuildScoreStore"), NumScoreEntries, ScoreBuildMinBatchSize,
										[&Scores, &ScoreEntryIDs, &ScoreEntryValues, &ScoreHashTable](int32 i)
										{
											const int64 ID = Scores[i].ID;
											ScoreEntryIDs[i] = ID;
											ScoreEntryValues[i] = MoveTemp(Scores[i].Scores);
											ScoreHashTable.Add_Concurrent((uint32)MurmurFinalize64((uint64)ID), (uint32)i);
										});

									AsyncTask(ENamedThreads::GameThread,
										[WeakSelf, Epoch, DispatchStartSeconds, Scores = MoveTemp(Scores),
										 ScoreEntryIDs = MoveTemp(ScoreEntryIDs),
										 ScoreEntryValues = MoveTemp(ScoreEntryValues),
										 ScoreHashTable = MoveTemp(ScoreHashTable)]() mutable
										{
											// Bail if canceled
											TSharedPtr<FSemanticSearchAssetViewOverride> GTSelf = WeakSelf.Pin();
											if (!GTSelf || GTSelf->ActiveEpoch != Epoch)
											{
												return;
											}

											const int32 NumScores = Scores.Num();
											GTSelf->LastMatchedItems.Reset(NumScores);
											GTSelf->LastMatchedItems.SetNum(NumScores);
											constexpr int32 ExtractMinBatchSize = 8192;
											ParallelFor(TEXT("SemanticSearchCB_ExtractAssetData"), NumScores, ExtractMinBatchSize,
												[&Scores, &GTSelf](int32 i)
												{
													FScoreEntry& Entry = Scores[i];
													if (Entry.Item.IsValid())
													{
														Entry.Item->GetItem().Legacy_TryGetAssetData(Entry.AssetData);
													}
													GTSelf->LastMatchedItems[i] = MoveTemp(Entry.Item);
												});
											FSearchResultScoreStore::Get().Set(MoveTemp(ScoreEntryIDs), MoveTemp(ScoreEntryValues), MoveTemp(ScoreHashTable));

											// Eager-fetch captions + keywords for the tooltip. DDC hits are cheap;
											// a miss triggers a rebuild but indexed assets should be cached.
											DispatchBackgroundOrInline(
												[Scores = MoveTemp(Scores)]() mutable
												{
													FAssetProcessorManager& Manager = FAssetProcessorManager::Get();
													for (FScoreEntry& Entry : Scores)
													{
														if (!Entry.AssetData.IsValid())
														{
															continue;
														}
														const int64 AssetID = Entry.ID;
														Manager.GetCaptionData(Entry.AssetData,
															[AssetID](FAssetCaptionResult&& Result, FString&& Error, EAssetIndexFailureReason Reason)
															{
																if (!Error.IsEmpty())
																{
																	return;
																}
																AsyncTask(ENamedThreads::GameThread,
																	[AssetID, Caption = MoveTemp(Result.Caption), Keywords = MoveTemp(Result.Keywords)]() mutable
																	{
																		FSearchResultScoreStore::Get().UpdateMetadata(AssetID, MoveTemp(Caption), MoveTemp(Keywords));
																	});
															});
													}
												});

											GTSelf->bHasActiveResults = true;

											if (!GTSelf->LastMatchedItems.IsEmpty())
											{
												UE_LOGF(LogSemanticSearchCB, Verbose, "Publish: publishing %d matching items (Epoch=%u, Query='%ls', Similarity=%ls)",
													GTSelf->LastMatchedItems.Num(), GTSelf->ActiveEpoch, *GTSelf->ActiveQuery.ToString(), GTSelf->bIsSimilaritySearch ? TEXT("yes") : TEXT("no"));
												GTSelf->bHasInFlightRequest = false;
												GTSelf->PublishMatchingItems();
											}

											const double ElapsedMs = (FPlatformTime::Seconds() - DispatchStartSeconds) * 1000.0;
											UE_LOGF(LogSemanticSearchCB, Verbose, "DispatchHybridSearch: end-to-end latency %.4f ms (Epoch=%u, Query='%ls', Published=%d)",
												ElapsedMs, Epoch, *GTSelf->ActiveQuery.ToString(), GTSelf->LastMatchedItems.Num());
										});
								});
					});
			});
		}

		void PublishMatchingItems()
		{
			PublishResults(MakeArrayView(LastMatchedItems));
		}

		FText ActiveQuery;
		TArray<TSharedPtr<FAssetViewItem>> CurrentItems;
		TArray<TSharedPtr<FAssetViewItem>> LastMatchedItems;
		TOptional<TArray<float>> PendingQueryEmbedding;
		uint32 PendingQueryEmbeddingEpoch = 0;
		uint32 ActiveEpoch = 0;
		bool bHasInFlightRequest = false;
		bool bHasActiveResults = false;
		bool bIsSimilaritySearch = false;
		bool bAllKnownItemsAvailable = false;
	};

	// ---------------------------------------------------------------------------

	class FSemanticSearchCBOverride : public IAssetSearchOverride
	{
	public:
		virtual TSharedRef<ICBSearchOverrideInstance> MakePerViewOverride() const override
		{
			return MakeShared<FSemanticSearchAssetViewOverride>();
		}
	};

	// ---------------------------------------------------------------------------

	TSharedRef<SWidget> FSemanticSearchAssetViewOverride::GetOverrideModeToggleWidget()
	{
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverOnlyButton")
			.ContentPadding(FMargin(4.f, 0.f))
			.ToolTipText(NSLOCTEXT("SemanticSearch", "ToggleButtonTooltip", "Toggle semantic search"))
			.OnClicked_Lambda([this]()
			{
				ToggleOverride();
				bIsAIActive = IsOverrideActive();
				if (bIsAIActive)
				{
					RequestQuickFrontendListRefresh();
				}
				else
				{
					// Tile badges visibility-poll the store; clearing it hides them on next paint.
					FSearchResultScoreStore::Get().Clear();
				}
				return FReply::Handled();
			})
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(NSLOCTEXT("SemanticSearch", "ToggleButtonLabel", "AI"))
				.ColorAndOpacity_Lambda([this]()
				{
					return IsOverrideActive()
						? FSlateColor(FLinearColor(0.f, 0.8f, 1.f))
						: FSlateColor::UseForeground();
				})
			];
	}

} // namespace Private

// ---------------------------------------------------------------------------

void RegisterContentBrowserExtension()
{
	FContentBrowserModule& ContentBrowserModule =
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	const bool bRegistered = ContentBrowserModule.RegisterAssetSearchOverride(MakeUnique<Private::FSemanticSearchCBOverride>());
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	ensureMsgf(bRegistered, TEXT("SemanticSearch: failed to register asset search override — another override is already active."));
}

void UnregisterContentBrowserExtension()
{
	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		ContentBrowserModule->UnregisterAssetSearchOverride();
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	}
}

} // namespace UE::SemanticSearch::ContentBrowser

// ---------------------------------------------------------------------------
// Index stats widget in the NavigationBar, shown alongside "X items"
// ---------------------------------------------------------------------------

static FDelayedAutoRegisterHelper SemanticSearchStatsNavBarExtension(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
		static const FName NavigationBarName("ContentBrowser.NavigationBar");
		if (UToolMenu* NavBar = UToolMenus::Get()->ExtendMenu(NavigationBarName))
		{
			FToolMenuSection& StatsSection = NavBar->FindOrAddSection(
				"SemanticSearchStats", {},
				FToolMenuInsert("AssetViewInfo", EToolMenuInsertType::After));
			StatsSection.Alignment = EToolMenuSectionAlign::Last;

			StatsSection.AddDynamicEntry(
				"SemanticSearchStatsEntry",
				FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					const UContentBrowserToolbarMenuContext* Context =
						InSection.FindContext<UContentBrowserToolbarMenuContext>();
					if (!Context || !Context->AssetView.IsValid())
					{
						return;
					}

					auto GetUnindexedCount = []() -> int32
					{
						const UE::SemanticSearch::FSemanticSearchIndexStats Stats = UE::SemanticSearch::ISemanticSearchModule::Get().GetIndexStats();
						return FMath::Max(0, Stats.SupportedAssetCount - Stats.VectorCount - Stats.FailedCount - Stats.PreProcessorFailedCount);
					};

					auto IsNotIndexing = []() -> bool
					{
						return !UE::SemanticSearch::ISemanticSearchModule::Get().GetIndexStats().bIsIndexing;
					};

					auto IndexingVisibility = []() -> EVisibility
					{
						return UE::SemanticSearch::ISemanticSearchModule::Get().GetIndexStats().bIsIndexing
							? EVisibility::Visible
							: EVisibility::Collapsed;
					};

					auto MakeRefreshButton = [](const FSlateBrush* Icon) -> TSharedRef<SWidget>
					{
						return SNew(SImage)
							.Image(Icon)
							.DesiredSizeOverride(FVector2D(12.f, 12.f));
					};

					const FSlateBrush* RefreshIcon = FAppStyle::GetBrush("SourceControl.Actions.Refresh");

					FToolMenuEntry& Entry = InSection.AddEntry(
						FToolMenuEntry::InitWidget(
							"SemanticSearchStatsEntry",
							SNew(SBox)
							.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
							.Visibility_Lambda([]()
							{
								return UE::SemanticSearch::ContentBrowser::bIsAIActive
									? EVisibility::Visible
									: EVisibility::Collapsed;
							})
							[
								SNew(SHorizontalBox)

								// "Semantic Search Index:"
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.TextStyle(FAppStyle::Get(), "SmallText")
									.Text(NSLOCTEXT("SemanticSearch", "IndexLabel", "Semantic Search Index:"))
								]

								// "{N} indexed" + reindex-all refresh icon
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4.f, 0.f, 0.f, 0.f)
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
										.TextStyle(FAppStyle::Get(), "SmallText")
										.Text_Lambda([]() -> FText
										{
											const int32 Count = UE::SemanticSearch::ISemanticSearchModule::Get().GetIndexStats().VectorCount;
											return FText::Format(NSLOCTEXT("SemanticSearch", "IndexedCount", "{0} indexed"), FText::AsNumber(Count));
										})
									]
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									.Padding(2.f, 0.f, 0.f, 0.f)
									[
										SNew(SButton)
										.ButtonStyle(FAppStyle::Get(), "HoverOnlyButton")
										.ContentPadding(FMargin(2.f, 0.f))
										.ToolTipText(NSLOCTEXT("SemanticSearch", "ReindexAllTooltip", "Re-index all assets from scratch"))
										.IsEnabled_Lambda(IsNotIndexing)
										.OnClicked_Lambda([]()
										{
											// Flip the button to disabled immediately; async enumeration + DDC
											// setup in IndexAllAssets would otherwise leave it clickable.
											UE::SemanticSearch::ISemanticSearchModule::Get().SetIndexingPending();
											UE::SemanticSearch::ISemanticSearchModule::Get().IndexAllAssets(/*bForceBuild=*/true);
											return FReply::Handled();
										})
										[
											MakeRefreshButton(RefreshIcon)
										]
									]
								]

								// " | {N} unindexed" + index-unindexed refresh icon
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4.f, 0.f, 0.f, 0.f)
								[
									SNew(SHorizontalBox)
									.Visibility_Lambda([GetUnindexedCount]()
									{
										return GetUnindexedCount() > 0
											? EVisibility::Visible
											: EVisibility::Collapsed;
									})
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
										.TextStyle(FAppStyle::Get(), "SmallText")
										.Text_Lambda([]() -> FText
										{
											const UE::SemanticSearch::FSemanticSearchIndexStats Stats = UE::SemanticSearch::ISemanticSearchModule::Get().GetIndexStats();
											const int32 Unindexed = FMath::Max(0, Stats.SupportedAssetCount - Stats.VectorCount - Stats.FailedCount - Stats.PreProcessorFailedCount);
											return FText::Format(NSLOCTEXT("SemanticSearch", "UnindexedCount", "| {0} unindexed"), FText::AsNumber(Unindexed));
										})
									]
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									.Padding(2.f, 0.f, 0.f, 0.f)
									[
										SNew(SButton)
										.ButtonStyle(FAppStyle::Get(), "HoverOnlyButton")
										.ContentPadding(FMargin(2.f, 0.f))
										.ToolTipText(NSLOCTEXT("SemanticSearch", "IndexUnindexedTooltip", "Index all unindexed assets"))
										.IsEnabled_Lambda(IsNotIndexing)
										.OnClicked_Lambda([]()
										{
											// Flip the button to disabled immediately — the async AR enumeration
											// + ContainsAsync/IsFailedAsync filter below would otherwise leave it
											// clickable until IndexAssets' OnBatchIndexingStarted fires later.
											UE::SemanticSearch::ISemanticSearchModule::Get().SetIndexingPending();
											IAssetRegistry& AR = IAssetRegistry::GetChecked();

											TArray<int64> CandidateIDs;
											TMap<int64, FAssetData> IDToAsset;
											AR.EnumerateAllAssets([&](const FAssetData& Asset)
											{
												if (!Asset.IsRedirector()
													&& UE::SemanticSearch::ISemanticSearchModule::IsInIndexedFolder(Asset))
												{
													int64 ID = UE::SemanticSearch::GetAssetIndexID(Asset);
													CandidateIDs.Add(ID);
													IDToAsset.Add(ID, Asset);
												}
												return true;
											});

											// Filter to IDs that are neither already indexed nor in the failed set
											// (failed assets have their own "Retry failed" button). Otherwise the progress
											// bar total gets inflated with assets that short-circuit via DDC hits,
											// desyncing it from the CB "unindexed" counter.
											TArray<int64> CandidateIDsCopy = CandidateIDs;
											UE::SemanticSearch::FHybridSearchIndex::Get().ContainsAsync(MoveTemp(CandidateIDsCopy),
												[CandidateIDs = MoveTemp(CandidateIDs), IDToAsset = MoveTemp(IDToAsset)]
												(TSet<int64>&& IndexedIDs) mutable
												{
													TArray<int64> RemainingIDs;
													RemainingIDs.Reserve(CandidateIDs.Num() - IndexedIDs.Num());
													for (int64 ID : CandidateIDs)
													{
														if (!IndexedIDs.Contains(ID))
														{
															RemainingIDs.Add(ID);
														}
													}

													UE::SemanticSearch::FHybridSearchIndex::Get().IsFailedAsync(RemainingIDs,
														[RemainingIDs, IDToAsset = MoveTemp(IDToAsset)]
														(TSet<int64>&& FailedIDs) mutable
														{
															TArray<FAssetData> AssetsToIndex;
															AssetsToIndex.Reserve(RemainingIDs.Num() - FailedIDs.Num());
															for (int64 ID : RemainingIDs)
															{
																if (FailedIDs.Contains(ID))
																{
																	continue;
																}
																if (const FAssetData* Asset = IDToAsset.Find(ID))
																{
																	AssetsToIndex.Add(*Asset);
																}
															}
															UE::SemanticSearch::ISemanticSearchModule::Get().IndexAssets(AssetsToIndex);
														});
												});
											return FReply::Handled();
										})
										[
											MakeRefreshButton(RefreshIcon)
										]
									]
								]

								// " | {N} failed" + retry-failed refresh icon
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4.f, 0.f, 0.f, 0.f)
								[
									SNew(SHorizontalBox)
									.Visibility_Lambda([]()
									{
										return UE::SemanticSearch::FHybridSearchIndex::Get().GetCachedIndexStats().FailedCount > 0
											? EVisibility::Visible
											: EVisibility::Collapsed;
									})
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
										.TextStyle(FAppStyle::Get(), "SmallText")
										.Text_Lambda([]() -> FText
										{
											const int32 Failed = UE::SemanticSearch::FHybridSearchIndex::Get().GetCachedIndexStats().FailedCount;
											return FText::Format(NSLOCTEXT("SemanticSearch", "FailedCount", "| {0} failed"), FText::AsNumber(Failed));
										})
									]
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									.Padding(2.f, 0.f, 0.f, 0.f)
									[
										SNew(SButton)
										.ButtonStyle(FAppStyle::Get(), "HoverOnlyButton")
										.ContentPadding(FMargin(2.f, 0.f))
										.ToolTipText(NSLOCTEXT("SemanticSearch", "RetryFailedTooltip", "Retry indexing all failed assets"))
										.IsEnabled_Lambda(IsNotIndexing)
										.OnClicked_Lambda([]()
										{
											// Flip the button to disabled immediately — IsRetryableFailedAsync
											// + IndexAssets below are async; without this the button stays
											// clickable during the spin-up window.
											UE::SemanticSearch::ISemanticSearchModule::Get().SetIndexingPending();
											IAssetRegistry& AR = IAssetRegistry::GetChecked();

											TArray<int64> CandidateIDs;
											TMap<int64, FAssetData> IDToAsset;
											AR.EnumerateAllAssets([&](const FAssetData& Asset)
											{
												if (!Asset.IsRedirector()
													&& UE::SemanticSearch::ISemanticSearchModule::IsInIndexedFolder(Asset))
												{
													int64 ID = UE::SemanticSearch::GetAssetIndexID(Asset);
													CandidateIDs.Add(ID);
													IDToAsset.Add(ID, Asset);
												}
												return true;
											});

											// Retryable only: pre-processor failures are permanent and must not be looped back
											// into indexing (they'd just fail the same way again).
											UE::SemanticSearch::FHybridSearchIndex::Get().IsRetryableFailedAsync(MoveTemp(CandidateIDs),
												[IDToAsset = MoveTemp(IDToAsset)](TSet<int64>&& FailedIDs)
												{
													TArray<FAssetData> AssetsToRetry;
													AssetsToRetry.Reserve(FailedIDs.Num());
													for (int64 ID : FailedIDs)
													{
														if (const FAssetData* Asset = IDToAsset.Find(ID))
														{
															AssetsToRetry.Add(*Asset);
														}
													}
													UE::SemanticSearch::ISemanticSearchModule::Get().IndexAssets(AssetsToRetry);
												});
											return FReply::Handled();
										})
										[
											MakeRefreshButton(RefreshIcon)
										]
									]
								]
								// Cancel indexing button (only visible while indexing) — rightmost slot
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4.f, 0.f, 0.f, 0.f)
								[
									SNew(SButton)
									.ButtonStyle(FAppStyle::Get(), "HoverOnlyButton")
									.ContentPadding(FMargin(2.f, 0.f))
									.ToolTipText(NSLOCTEXT("SemanticSearch", "CancelIndexingTooltip", "Cancel all in-flight indexing tasks and clear the queue"))
									.Visibility_Lambda(IndexingVisibility)
									.OnClicked_Lambda([]()
									{
										UE::SemanticSearch::ISemanticSearchModule::Get().CancelIndexing();
										return FReply::Handled();
									})
									[
										SNew(SImage)
										.Image(FAppStyle::GetBrush("Icons.X"))
										.DesiredSizeOverride(FVector2D(12.f, 12.f))
									]
								]
							],
							FText::GetEmpty()));
					Entry.WidgetData.StyleParams.VerticalAlignment = VAlign_Center;
				}));
		}
	});

// ---------------------------------------------------------------------------
// "Find Similar" context menu item for assets
// ---------------------------------------------------------------------------

static FDelayedAutoRegisterHelper SemanticSearchFindSimilarContextMenu(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
		if (UToolMenu* AssetMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu"))
		{
			FToolMenuSection& Section = AssetMenu->FindOrAddSection("SemanticSearchActions");
			Section.AddDynamicEntry(
				"FindSimilarEntry",
				FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					const UContentBrowserAssetContextMenuContext* Context =
						InSection.FindContext<UContentBrowserAssetContextMenuContext>();
					if (!Context || Context->SelectedAssets.Num() != 1)
					{
						return;
					}

					if (!UE::SemanticSearch::ContentBrowser::bIsAIActive)
					{
						return;
					}

					const FAssetData& SelectedAsset = Context->SelectedAssets[0];
					if (UE::SemanticSearch::FHybridSearchIndex::Get().GetCachedIndexStats().VectorCount == 0)
					{
						return;
					}

					InSection.AddMenuEntry(
						"FindSimilar",
						NSLOCTEXT("SemanticSearch", "FindSimilarLabel", "Find Similar"),
						NSLOCTEXT("SemanticSearch", "FindSimilarTooltip", "Find semantically similar assets using the embedding index"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([SelectedAsset]()
						{
							FContentBrowserModule& CBModule = FModuleManager::GetModuleChecked<FContentBrowserModule>("ContentBrowser");
							FString Command = FString::Printf(TEXT("/find-similar %s"), *SelectedAsset.GetObjectPathString());
							CBModule.Get().SetSearchText(FText::FromString(Command));
						}))
					);
				}));
		}
	});

