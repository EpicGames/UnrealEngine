// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISemanticSearchEditorIntegrationsModule.h"

#include "ContentBrowserModule.h"
#include "ContentBrowser/SearchResultScoreStore.h"
#include "ContentBrowser/SemanticSearchCB.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"
#include "Widgets/SSemanticSearchIndexDialog.h"

#define LOCTEXT_NAMESPACE "SemanticSearchEditorIntegrations"

namespace UE::SemanticSearch
{
	class FSemanticSearchEditorIntegrationsModule : public ISemanticSearchEditorIntegrationsModule
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

	private:
		void RegisterToolsMenuEntry();
	};

	void FSemanticSearchEditorIntegrationsModule::StartupModule()
	{
		ContentBrowser::RegisterContentBrowserExtension();
		ContentBrowser::RegisterScoreBadgeGenerator();

		UToolMenus::RegisterStartupCallback(
			FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FSemanticSearchEditorIntegrationsModule::RegisterToolsMenuEntry));
	}

	void FSemanticSearchEditorIntegrationsModule::ShutdownModule()
	{
		ContentBrowser::UnregisterScoreBadgeGenerator();
		ContentBrowser::UnregisterContentBrowserExtension();
	}

	void FSemanticSearchEditorIntegrationsModule::RegisterToolsMenuEntry()
	{
		UToolMenu* ToolsMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Tools");
		if (!ToolsMenu)
		{
			return;
		}

		FToolMenuSection& Section = ToolsMenu->FindOrAddSection("SemanticSearch");
		Section.AddMenuEntry(
			"SemanticSearchIndex",
			LOCTEXT("MenuLabel", "Semantic Search"),
			LOCTEXT("MenuTooltip", "View and manage the semantic search index"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([]()
			{
				SSemanticSearchIndexDialog::Open();
			}))
		);
	}
}

IMPLEMENT_MODULE(UE::SemanticSearch::FSemanticSearchEditorIntegrationsModule, SemanticSearchEditorIntegrations)

#undef LOCTEXT_NAMESPACE
