// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSemanticSearchIndexDialog.h"

#include "AssetProcessorManager.h"
#include "HybridSearchIndex.h"
#include "Framework/Application/SlateApplication.h"
#include "Containers/Ticker.h"
#include "Widgets/SWindow.h"
#include "Settings/SemanticSearchSettings.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SemanticSearchIndexDialog"

namespace UE::SemanticSearch
{

static FString FormatMemorySize(int64 Bytes)
{
	if (Bytes < 1024)
	{
		return FString::Printf(TEXT("%lld B"), Bytes);
	}
	if (Bytes < 1024 * 1024)
	{
		return FString::Printf(TEXT("%.1f KB"), Bytes / 1024.0);
	}
	if (Bytes < 1024LL * 1024 * 1024)
	{
		return FString::Printf(TEXT("%.1f MB"), Bytes / (1024.0 * 1024.0));
	}
	return FString::Printf(TEXT("%.2f GB"), Bytes / (1024.0 * 1024.0 * 1024.0));
}

void SSemanticSearchIndexDialog::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(500.0f)
		.Padding(16.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(StatsBox, SVerticalBox)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ThrobberBox, SVerticalBox)
				.Visibility(EVisibility::Collapsed)
			]
		]
	];

	// Bind to index change notifications for auto-refresh
	IndexChangedHandle = FHybridSearchIndex::Get().OnIndexChanged().AddSP(
		this, &SSemanticSearchIndexDialog::OnIndexChanged);

	RefreshStats();
}

SSemanticSearchIndexDialog::~SSemanticSearchIndexDialog()
{
	if (IndexChangedHandle.IsValid())
	{
		FHybridSearchIndex::Get().OnIndexChanged().Remove(IndexChangedHandle);
	}
}

void SSemanticSearchIndexDialog::RefreshStats()
{
	CachedStats = ISemanticSearchModule::Get().GetIndexStats();

	if (!StatsBox.IsValid())
	{
		return;
	}

	StatsBox->ClearChildren();

	auto AddRow = [this](const FText& Label, const FText& Value)
	{
		StatsBox->AddSlot()
		.AutoHeight()
		.Padding(0, 2)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.55f)
			[
				SNew(STextBlock)
				.Text(Label)
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(0.45f)
			[
				SNew(STextBlock)
				.Text(Value)
			]
		];
	};

	const UEnum* IndexTypeEnum = StaticEnum<ESemanticSearchIndexType>();
	const FText IndexTypeDisplayName = IndexTypeEnum
		? IndexTypeEnum->GetDisplayNameTextByValue(static_cast<int64>(CachedStats.IndexType))
		: FText::FromString(TEXT("Unknown"));

	AddRow(LOCTEXT("IndexType", "Index Type"), IndexTypeDisplayName);
	AddRow(LOCTEXT("VectorCount", "Vector Index Count"), FText::AsNumber(CachedStats.VectorCount));
	AddRow(LOCTEXT("BM25Count", "BM25 Index Count"), FText::AsNumber(CachedStats.BM25Count));
	AddRow(LOCTEXT("Dimension", "Embedding Dimension"), FText::AsNumber(CachedStats.Dimension));
	AddRow(LOCTEXT("VectorMemoryEst", "Est. Vector Index Memory Usage"), FText::FromString(FormatMemorySize(CachedStats.EstimatedVectorMemoryBytes)));
	AddRow(LOCTEXT("BM25MemoryEst", "Est. BM25 Index Memory Usage"), FText::FromString(FormatMemorySize(CachedStats.EstimatedBM25MemoryBytes)));
	AddRow(LOCTEXT("Trained", "Trained"), FText::FromString(CachedStats.bIsTrained ? TEXT("Yes") : TEXT("No")));
	AddRow(LOCTEXT("SupportedAssets",        "Supported Assets"),    FText::AsNumber(CachedStats.SupportedAssetCount));
	AddRow(LOCTEXT("IndexedCount",           "Indexed"),             FText::AsNumber(CachedStats.VectorCount));
	AddRow(LOCTEXT("PreProcessorFailCount",  "Skipped (ineligible)"), FText::AsNumber(CachedStats.PreProcessorFailedCount));
	AddRow(LOCTEXT("FailedCount",            "Failed (retryable)"),  FText::AsNumber(CachedStats.FailedCount));

	// Separator
	StatsBox->AddSlot()
	.AutoHeight()
	.Padding(0, 8)
	[
		SNew(SSeparator)
	];

	// Add a switch button for each alternative index type
	if (IndexTypeEnum)
	{
		for (int32 i = 0; i < IndexTypeEnum->NumEnums() - 1; ++i) // -1 to skip _MAX
		{
			const ESemanticSearchIndexType Type = static_cast<ESemanticSearchIndexType>(IndexTypeEnum->GetValueByIndex(i));
			if (Type == CachedStats.IndexType)
			{
				continue;
			}

			const FText ButtonLabel = FText::Format(
				LOCTEXT("SwitchToFmt", "Switch to {0}"),
				IndexTypeEnum->GetDisplayNameTextByIndex(i));

			StatsBox->AddSlot()
			.AutoHeight()
			.Padding(0, 4)
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.Text(ButtonLabel)
				.OnClicked_Lambda([this, Type]()
				{
					ISemanticSearchModule::Get().SwitchIndexType(Type);
					ShowThrobber();
					return FReply::Handled();
				})
			];
		}
	}

	// Retrain button — only for quantized indices that are already trained
	if (CachedStats.IndexType != ESemanticSearchIndexType::Flat && CachedStats.bIsTrained)
	{
		StatsBox->AddSlot()
		.AutoHeight()
		.Padding(0, 4)
		.HAlign(HAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("RetrainIndex", "Retrain Index"))
			.OnClicked_Lambda([this]()
			{
				ISemanticSearchModule::Get().RetrainIndex();
				ShowThrobber();
				return FReply::Handled();
			})
		];
	}
}

void SSemanticSearchIndexDialog::ShowThrobber()
{
	if (!ThrobberBox.IsValid())
	{
		return;
	}

	ThrobberBox->ClearChildren();
	ThrobberBox->AddSlot()
	.AutoHeight()
	.Padding(0, 8)
	.HAlign(HAlign_Center)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SThrobber)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(8, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SwitchingIndex", "Switching index type..."))
		]
	];

	ThrobberBox->SetVisibility(EVisibility::Visible);
}

void SSemanticSearchIndexDialog::OnIndexChanged(bool bSuccess)
{
	if (!bSuccess && ThrobberBox.IsValid())
	{
		// Replace throbber with error message
		ThrobberBox->ClearChildren();
		ThrobberBox->AddSlot()
		.AutoHeight()
		.Padding(0, 8)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SwitchFailed", "Switch failed. Check the log for details."))
			.ColorAndOpacity(FLinearColor(1.f, 0.3f, 0.3f))
		];

		// Auto-hide after 5 seconds
		TWeakPtr<SVerticalBox> WeakThrobberBox = ThrobberBox;
		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda([WeakThrobberBox](float) -> bool
			{
				if (TSharedPtr<SVerticalBox> Box = WeakThrobberBox.Pin())
				{
					Box->SetVisibility(EVisibility::Collapsed);
				}
				return false; // one-shot
			}), 10.0f);
	}
	else if (ThrobberBox.IsValid())
	{
		ThrobberBox->SetVisibility(EVisibility::Collapsed);
	}
	RefreshStats();
}

void SSemanticSearchIndexDialog::Open()
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("WindowTitle", "Semantic Search"))
		.SizingRule(ESizingRule::Autosized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SSemanticSearchIndexDialog)
		];

	FSlateApplication::Get().AddWindow(Window);
}

} // namespace UE::SemanticSearch

#undef LOCTEXT_NAMESPACE
