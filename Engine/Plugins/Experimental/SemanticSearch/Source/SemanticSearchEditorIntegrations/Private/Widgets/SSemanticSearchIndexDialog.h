// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISemanticSearchModule.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"

class STextBlock;

namespace UE::SemanticSearch
{

class SSemanticSearchIndexDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSemanticSearchIndexDialog) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SSemanticSearchIndexDialog();

	/** Open the dialog as a standalone window. */
	static void Open();

private:
	void RefreshStats();
	void ShowThrobber();
	void OnIndexChanged(bool bSuccess);

	FSemanticSearchIndexStats CachedStats;
	TSharedPtr<SVerticalBox> StatsBox;
	TSharedPtr<SVerticalBox> ThrobberBox;
	TSharedPtr<STextBlock> StatusText;
	FDelegateHandle IndexChangedHandle;
};

} // namespace UE::SemanticSearch
