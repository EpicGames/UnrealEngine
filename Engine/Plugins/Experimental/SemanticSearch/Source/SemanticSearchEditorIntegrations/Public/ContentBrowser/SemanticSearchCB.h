// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::SemanticSearch::ContentBrowser
{
	SEMANTICSEARCHEDITORINTEGRATIONS_API void RegisterContentBrowserExtension();
	SEMANTICSEARCHEDITORINTEGRATIONS_API void UnregisterContentBrowserExtension();
}