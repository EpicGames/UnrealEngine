// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::SemanticSearch
{
	class ISemanticSearchEditorIntegrationsModule : public IModuleInterface
	{
	public:
		static ISemanticSearchEditorIntegrationsModule& Get()
		{
			static const FName ModuleName = "SemanticSearchEditorIntegrations";
			return FModuleManager::LoadModuleChecked<ISemanticSearchEditorIntegrationsModule>(ModuleName);
		}
	};
}
