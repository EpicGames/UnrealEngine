// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

/**
 * Editor module responsible for integrating the semantic search system into
 * editor-facing surfaces such as the Content Browser.
 *
 * UI and higher-level editor logic lives here; core search logic lives in SemanticSearch.
 */
public class SemanticSearchEditorIntegrations : ModuleRules
{
	public SemanticSearchEditorIntegrations(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Analytics",
				"ContentBrowser",
				"ContentBrowserData",
				"CoreUObject",
				"Engine",
				"SemanticSearch",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd",
			});

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			});
	}
}
